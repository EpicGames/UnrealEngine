// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class DataHierarchyEditor : ModuleRules
{
    public DataHierarchyEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core", 
                "CoreUObject",
                "EditorWidgets",
                "SlateCore"
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Engine",
                "Slate",
                "UnrealEd",
                "ToolMenus",
				"ToolWidgets",
                "InputCore"
            }
        );
    }
}