// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/CoreStyle.h"
#include "Framework/Commands/Commands.h"

class DATAHIERARCHYEDITOR_API FDataHierarchyEditorCommands : public TCommands<FDataHierarchyEditorCommands>
{
public:
	
	FDataHierarchyEditorCommands()
		: TCommands<FDataHierarchyEditorCommands>( TEXT("DataHierarchyEditorCommands"), NSLOCTEXT("DataHierarchyEditorCommands", "Data Hierarchy Editor Commands", "Data Hierarchy Editor Commands"), NAME_None, "DataHierarchyEditorStyle" )
	{
	}

	virtual ~FDataHierarchyEditorCommands() override = default;

	virtual void RegisterCommands() override;	

	TSharedPtr<FUICommandInfo> FindInHierarchy;
};
