// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "UObject/NameTypes.h"
#include "DataHierarchyEditorMisc.generated.h"

class SDataHierarchyEditor;

namespace UE::DataHierarchyEditor
{
	DATAHIERARCHYEDITOR_API FName GetUniqueName(FName CandidateName, const TSet<FName>& ExistingNames);
}

UCLASS()
class UDataHierarchyEditorMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<SDataHierarchyEditor> Widget;
};