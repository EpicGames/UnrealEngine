// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

#define UE_API DATAHIERARCHYEDITOR_API

/** Manages the style which provides resources for Hierarchy Editor widgets. */
class FDataHierarchyEditorStyle : public FSlateStyleSet
{
public:
	static UE_API void Register();
	static UE_API void Unregister();
	static UE_API void Shutdown();

	/** reloads textures used by slate renderer */
	static UE_API void ReloadTextures();

	/** @return The Slate style set for Hierarchy Editor widgets */
	static UE_API const FDataHierarchyEditorStyle& Get();

	static UE_API void ReinitializeStyle();

private:	
	UE_API FDataHierarchyEditorStyle();

	UE_API void InitDataHierarchyEditor();

	static UE_API TSharedPtr<FDataHierarchyEditorStyle> DataHierarchyEditorStyle;
};

#undef UE_API
