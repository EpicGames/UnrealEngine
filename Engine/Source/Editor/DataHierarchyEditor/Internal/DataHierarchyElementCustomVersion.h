// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#define UE_API DATAHIERARCHYEDITOR_API

struct FGuid;

struct FDataHierarchyElementCustomVersion
{
	enum Type
	{
		BeforeCustomVersionWasAdded = 0, // Before any version changes were made in the plugin

		MetaDataIntroduction, // PostLoad moving hard coded section references on categories into the metadata system
		
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid Guid;

	private:
		FDataHierarchyElementCustomVersion() {}
};

#undef UE_API
