// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SDropTarget.h"
#include "UObject/ObjectKey.h"
#include "IDetailsView.h"
#include "DataHierarchyEditorStyle.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Styling/SlateTypes.h"
#include "DataHierarchyViewModelBase.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Misc/NotifyHook.h"
#include "UObject/GCObject.h"

#define UE_API DATAHIERARCHYEDITOR_API

class UDataHierarchyViewModelBase;
class UHierarchySection;
struct FHierarchyElementViewModel;
struct FHierarchySectionViewModel;

class SHierarchySection : public SCompoundWidget
{
	DECLARE_DELEGATE_OneParam(FOnSectionActivated, TSharedPtr<FHierarchySectionViewModel> SectionViewModel)
	
	SLATE_BEGIN_ARGS(SHierarchySection)
		{}
		SLATE_ATTRIBUTE(ECheckBoxState, IsSectionActive)
		SLATE_EVENT(FOnSectionActivated, OnSectionActivated)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, TSharedPtr<struct FHierarchySectionViewModel> InSection);
	UE_API virtual ~SHierarchySection() override;

	UE_API void TryEnterEditingMode() const;

	UE_API TSharedPtr<struct FHierarchySectionViewModel> GetSectionViewModel();
private:
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API int32 PaintDropIndicator(const FPaintArgs& Args, const FGeometry& Geometry, FSlateRect SlateRect, FSlateWindowElementList& OutDrawElements, int32 INT32, const FWidgetStyle& WidgetStyle, bool bParentEnabled) const;
	UE_API int32 OnPaintDropIndicator(EItemDropZone ItemDropZone, const FPaintArgs& Args, const FGeometry& Geometry, FSlateRect SlateRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& WidgetStyle, bool bParentEnabled) const;

	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	UE_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	
	UE_API bool OnCanAcceptDrop(TSharedPtr<FDragDropOperation> DragDropOperation, EItemDropZone ItemDropZone) const;
	UE_API FReply OnDroppedOn(const FGeometry&, const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const;
	
	UE_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	UE_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	UE_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	
	UE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	UE_API UHierarchySection* TryGetSectionData() const;

	const FSlateBrush* GetImageBrush() const;
	EVisibility GetImageVisibility() const;

	UE_API FText GetText() const;
	UE_API FText GetTooltipText() const;
	UE_API void OnRenameSection(const FText& Text, ETextCommit::Type CommitType) const;
	UE_API bool OnVerifySectionRename(const FText& NewName, FText& OutTooltip) const;

	UE_API bool IsSectionSelected() const;
	UE_API bool IsSectionReadOnly() const;
	UE_API ECheckBoxState GetSectionCheckState() const;
	UE_API void OnSectionCheckChanged(ECheckBoxState NewState);
	UE_API EActiveTimerReturnType ActivateSectionIfDragging(double CurrentTime, float DeltaTime) const;

	UE_API const FSlateBrush* GetDropIndicatorBrush(EItemDropZone ItemDropZone) const;

	/** @return the zone (above, onto, below) based on where the user is hovering over within the row */
	UE_API EItemDropZone ZoneFromPointerPosition(UE::Slate::FDeprecateVector2DParameter LocalPointerPos, UE::Slate::FDeprecateVector2DParameter LocalSize);

private:
	TSharedPtr<SMenuAnchor> MenuAnchor;
	TSharedPtr<SCheckBox> CheckBox;
	TSharedPtr<SInlineEditableTextBlock> InlineEditableTextBlock;
	TWeakObjectPtr<UDataHierarchyViewModelBase> HierarchyViewModel;
	TWeakPtr<FHierarchySectionViewModel> SectionViewModelWeak;
private:
	TAttribute<ECheckBoxState> IsSectionActive;
	FOnSectionActivated OnSectionActivatedDelegate;
	
	mutable bool bDraggedOn = false;
	
	mutable TOptional<EItemDropZone> CurrentItemDropZone;
};

class SDataHierarchyEditor : public SCompoundWidget, public FNotifyHook
{
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnGenerateRowContentWidget, TSharedRef<FHierarchyElementViewModel> HierarchyElement);
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnGenerateCustomDetailsPanelNameWidget, TSharedPtr<FHierarchyElementViewModel> HierarchyElement);

	SLATE_BEGIN_ARGS(SDataHierarchyEditor)
		: _bReadOnly(true)
	{}
		SLATE_ARGUMENT(bool, bReadOnly)
		SLATE_EVENT(FOnGenerateRowContentWidget, OnGenerateRowContentWidget)
		SLATE_EVENT(FOnGenerateCustomDetailsPanelNameWidget, OnGenerateCustomDetailsPanelNameWidget)
	SLATE_END_ARGS()

public:
	UE_API void Construct(const FArguments& InArgs, TObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel);
	UE_API virtual ~SDataHierarchyEditor() override;

	UE_API void RefreshSourceItems();
	UE_API void RefreshAllViews(bool bFullRefresh = false);
	UE_API void RequestRefreshAllViewsNextFrame(bool bFullRefresh = false);
	UE_API void RefreshSourceView(bool bFullRefresh = false) const;
	UE_API void RequestRefreshSourceViewNextFrame(bool bFullRefresh = false);
	UE_API void RefreshHierarchyView(bool bFullRefresh = false) const;
	UE_API void RequestRefreshHierarchyViewNextFrame(bool bFullRefresh = false);
	UE_API void RefreshSectionsView();
	UE_API void RequestRefreshSectionsViewNextFrame();

	UE_API void NavigateToHierarchyElement(FHierarchyElementIdentity Identity) const;
	UE_API void NavigateToHierarchyElement(TSharedPtr<FHierarchyElementViewModel> Item) const;
	UE_API bool IsItemSelected(TSharedPtr<FHierarchyElementViewModel> Item) const;
	
private:
	// need to do this to enable focus so we can handle shortcuts
	virtual bool SupportsKeyboardFocus() const override { return true; }
	
	UE_API TSharedRef<ITableRow> GenerateSourceItemRow(TSharedPtr<FHierarchyElementViewModel> HierarchyItem, const TSharedRef<STableViewBase>& TableViewBase);
	UE_API TSharedRef<ITableRow> GenerateHierarchyItemRow(TSharedPtr<FHierarchyElementViewModel> HierarchyItem, const TSharedRef<STableViewBase>& TableViewBase);

	UE_API bool FilterForSourceSection(TSharedPtr<const FHierarchyElementViewModel> ItemViewModel) const;
private:
	UE_API void Reinitialize();

	UE_API void BindToHierarchyRootViewModel();
	UE_API void UnbindFromHierarchyRootViewModel() const;
	
	/** Source items reflect the base, unedited status of items to edit into a hierarchy */
	UE_API const TArray<TSharedPtr<FHierarchyElementViewModel>>& GetSourceItems() const;
	
	UE_API bool IsDetailsPanelEditingAllowed() const;
	
	UE_API void ClearSourceItems() const;
	
	UE_API void RequestRenameSelectedItem();
	UE_API bool CanRequestRenameSelectedItem() const;

	UE_API void DeleteItems(TArray<TSharedPtr<FHierarchyElementViewModel>> ItemsToDelete) const;
	UE_API void DeleteSelectedHierarchyItems() const;
	UE_API bool CanDeleteSelectedElements() const;

	UE_API void NavigateToMatchingHierarchyElementFromSelectedSourceElement() const;
	UE_API bool CanNavigateToMatchingHierarchyElementFromSelectedSourceElement() const;
	
	UE_API void OnHierarchySectionActivated(TSharedPtr<FHierarchySectionViewModel> Section);
	UE_API void OnSourceSectionActivated(TSharedPtr<FHierarchySectionViewModel> Section);
	UE_API void OnHierarchySectionAdded(TSharedPtr<FHierarchySectionViewModel> AddedSection);
	UE_API void OnHierarchySectionDeleted(TSharedPtr<FHierarchySectionViewModel> DeletedSection);

	UE_API void SetActiveSourceSection(TSharedPtr<struct FHierarchySectionViewModel>);
	UE_API TSharedPtr<FHierarchySectionViewModel> GetActiveSourceSection() const;
	UE_API UHierarchySection* GetActiveSourceSectionData() const;
	
	TSharedRef<SWidget> OnGetCustomAddContent();

	void OnSelectionChanged(TSharedPtr<FHierarchyElementViewModel> SelectedHierarchyElement, ESelectInfo::Type Type) const;

	UE_API void RunSourceSearch();
	UE_API void OnSourceSearchTextChanged(const FText& Text);
	UE_API void OnSourceSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType);
	UE_API void OnSearchButtonClicked(SSearchBox::SearchDirection SearchDirection);

	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	UE_API FReply OnHierarchyRowDragDetected(TSharedRef<FHierarchyElementViewModel> DraggedRow, const FGeometry& Geometry, const FPointerEvent& PointerEvent);
	UE_API FReply OnSourceRowDragDetected(TSharedRef<FHierarchyElementViewModel> DraggedRow, const FGeometry& Geometry, const FPointerEvent& PointerEvent);
	UE_API FReply HandleMultiDragDetected(const TArray<TSharedPtr<FHierarchyElementViewModel>>& ItemsToDrag, bool bIsSource);

	UE_API FReply OnAddCategoryClicked() const;
	UE_API FReply OnAddSectionClicked() const;

	UE_API TSharedPtr<SWidget> SummonContextMenuForSelectedRows(bool bFromHierarchy) const;

	struct FSearchItem
	{
		TArray<TSharedPtr<FHierarchyElementViewModel>> Path;

		TSharedPtr<FHierarchyElementViewModel> GetEntry() const
		{
			return Path.Num() > 0 ? 
				Path[Path.Num() - 1] : 
				nullptr;
		}

		bool operator==(const FSearchItem& Item) const
		{
			return Path == Item.Path;
		}
	};

	/** This will recursively generated parent chain paths for all items within the given root. Used for expansion purposes. */
	UE_API void GenerateSearchItems(TSharedRef<FHierarchyElementViewModel> Root, TArray<TSharedPtr<FHierarchyElementViewModel>> ParentChain, TArray<FSearchItem>& OutSearchItems);
	UE_API void ExpandSourceSearchResults();
	UE_API void SelectNextSourceSearchResult();
	UE_API void SelectPreviousSourceSearchResult();
	UE_API TOptional<SSearchBox::FSearchResultData> GetSearchResultData() const;

	void ExpandEntriesByDefault(TSharedPtr<STreeView<TSharedPtr<FHierarchyElementViewModel>>> TreeView) const;

	/** Saves the expansion state of all items in the given tree view, keyed by FHierarchyElementIdentity when available, falling back to TObjectKey. */
	void SaveExpansionState(TSharedPtr<STreeView<TSharedPtr<FHierarchyElementViewModel>>> TreeView) const;
	/** Restores previously saved expansion state. Items not in the saved state fall back to IsExpandedByDefault(). */
	void RestoreExpansionState(TSharedPtr<STreeView<TSharedPtr<FHierarchyElementViewModel>>> TreeView) const;
	
	FHierarchyElementViewModel::FResultWithUserFeedback CanDropOnRoot(TSharedPtr<FHierarchyElementViewModel> DraggedItem) const;

	/** Callback functions for the root widget */
	UE_API FReply HandleHierarchyRootDrop(const FGeometry&, const FDragDropEvent& DragDropEvent) const;
	UE_API bool OnCanDropOnRoot(TSharedPtr<FDragDropOperation> DragDropOperation) const;
	UE_API void OnRootDragEnter(const FDragDropEvent& DragDropEvent) const;
	UE_API void OnRootDragLeave(const FDragDropEvent& DragDropEvent) const;
	UE_API FSlateColor GetRootIconColor() const;

	UE_API virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
private:
	/** A weak pointer to the hierarchy view model.
	 * An external view model created and managed by the implementer. */
	TWeakObjectPtr<UDataHierarchyViewModelBase> HierarchyViewModel;
	TArray<FSearchItem> SourceSearchResults;
	TOptional<FSearchItem> FocusedSearchResult;
	
	mutable TArray<TWeakPtr<FHierarchyElementViewModel>> SelectedDetailsPanelItemViewModels;
	mutable bool bIsUpdatingSelection = false;

private:
	TStrongObjectPtr<UHierarchyRoot> SourceRoot;
	TSharedPtr<FHierarchyRootViewModel> SourceRootViewModel;
	TSharedPtr<STreeView<TSharedPtr<FHierarchyElementViewModel>>> SourceTreeView;
	TSharedPtr<STreeView<TSharedPtr<FHierarchyElementViewModel>>> HierarchyTreeView;
	TStrongObjectPtr<UHierarchySection> AllSourceSection;
	TSharedPtr<FHierarchySectionViewModel> DefaultSourceSectionViewModel;
	TWeakPtr<struct FHierarchySectionViewModel> ActiveSourceSection;
	TSharedPtr<class SWrapBox> SourceSectionBox;
	TSharedPtr<class SWrapBox> HierarchySectionBox;
	TSharedPtr<SSearchBox> SourceSearchBox;
	TSharedPtr<IDetailsView> DetailsPanel;
private:
	FOnGenerateRowContentWidget OnGenerateRowContentWidget;
	FOnGenerateCustomDetailsPanelNameWidget OnGenerateCustomDetailsPanelNameWidget;
	TSharedPtr<FActiveTimerHandle> RefreshHierarchyViewNextFrameHandle;
	TSharedPtr<FActiveTimerHandle> RefreshSourceViewNextFrameHandle;
	TSharedPtr<FActiveTimerHandle> RefreshSectionsViewNextFrameHandle;

	/** Saved expansion state. Uses FHierarchyElementIdentity when available, falls back to TObjectKey for elements without a valid identity. */
	mutable TMap<FHierarchyElementIdentity, bool> SavedExpansionByIdentity;
	mutable TMap<TObjectKey<const UHierarchyElement>, bool> SavedExpansionByObjectKey;
};

class SHierarchyElement : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SHierarchyElement)
		: _Style(&FCoreStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockStyle") )
	{}
		SLATE_STYLE_ARGUMENT(FInlineEditableTextBlockStyle, Style)
		SLATE_EVENT(FIsSelected, IsSelected)
	SLATE_END_ARGS()

public:
	UE_API void Construct(const FArguments& InArgs, TSharedPtr<struct FHierarchyElementViewModel> InViewModel);

	UE_API void EnterEditingMode() const;
	UE_API bool OnVerifyRename(const FText& NewName, FText& OutTooltip) const;
	
	UE_API FText GetElementText() const;
	UE_API void OnRenameElement(const FText& NewText, ETextCommit::Type) const;
	
	FText GetTooltipText() const;
private:
	TWeakPtr<struct FHierarchyElementViewModel> ElementViewModelWeak;
	TSharedPtr<SInlineEditableTextBlock> InlineEditableTextBlock;
};

#undef UE_API
