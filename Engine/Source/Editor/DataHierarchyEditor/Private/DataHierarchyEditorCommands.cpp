// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataHierarchyEditorCommands.h"

#define LOCTEXT_NAMESPACE "DataHierarchyEditorCommands"

void FDataHierarchyEditorCommands::RegisterCommands()
{
	UI_COMMAND(FindInHierarchy, "Find in Hierarchy", "Find the element in the hierarchy", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
}

#undef LOCTEXT_NAMESPACE
