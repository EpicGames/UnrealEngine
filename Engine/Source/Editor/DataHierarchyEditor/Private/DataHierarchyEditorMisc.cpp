// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataHierarchyEditorMisc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataHierarchyEditorMisc)

FName UE::DataHierarchyEditor::GetUniqueName(FName CandidateName, const TSet<FName>& ExistingNames)
{
	// This utility function needs to generate a unique name while only considering the text portion of the name and
	// not the index, so generate names with 0 indices before using them for comparison.
	TSet<FName> ExistingNamesWithIndexZero;
	for (FName ExistingName : ExistingNames)
	{
		ExistingNamesWithIndexZero.Add(FName(ExistingName, 0));
	}
	FName CandidateNameWithIndexZero = FName(CandidateName, 0);

	if (ExistingNamesWithIndexZero.Contains(CandidateNameWithIndexZero) == false)
	{
		return CandidateName;
	}

	FString CandidateNameString = CandidateNameWithIndexZero.ToString();
	FString BaseNameString = CandidateNameString;
	if (CandidateNameString.Len() >= 3 && CandidateNameString.Right(3).IsNumeric())
	{
		BaseNameString = CandidateNameString.Left(CandidateNameString.Len() - 3);
	}

	FName UniqueName = FName(*BaseNameString);
	int32 NameIndex = 1;
	while (ExistingNamesWithIndexZero.Contains(UniqueName))
	{
		UniqueName = FName(*FString::Printf(TEXT("%s%03i"), *BaseNameString, NameIndex));
		NameIndex++;
	}

	return UniqueName;
}
