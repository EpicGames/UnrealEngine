// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataHierarchyEditorModule.h"

#include "DataHierarchyEditorCommands.h"
#include "DataHierarchyEditorStyle.h"

#define LOCTEXT_NAMESPACE "DataHierarchyEditor"

DEFINE_LOG_CATEGORY(LogDataHierarchyEditor)

void FDataHierarchyEditorModule::StartupModule()
{
    FDataHierarchyEditorStyle::Register();
	FDataHierarchyEditorCommands::Register();
}

void FDataHierarchyEditorModule::ShutdownModule()
{
	FDataHierarchyEditorCommands::Unregister();
    FDataHierarchyEditorStyle::Shutdown();
}

#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(FDataHierarchyEditorModule, DataHierarchyEditor)
