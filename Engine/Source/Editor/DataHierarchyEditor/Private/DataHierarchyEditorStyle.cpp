// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataHierarchyEditorStyle.h"

#include "Framework/Application/SlateApplication.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"

TSharedPtr<FDataHierarchyEditorStyle> FDataHierarchyEditorStyle::DataHierarchyEditorStyle = nullptr;

void FDataHierarchyEditorStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}

void FDataHierarchyEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}

void FDataHierarchyEditorStyle::Shutdown()
{
	Unregister();
	DataHierarchyEditorStyle.Reset();
}

void FDataHierarchyEditorStyle::InitDataHierarchyEditor()
{
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FSlateColor SelectorColor = FAppStyle::GetSlateColor("SelectorColor");
	const FSlateColor SelectionColor = FAppStyle::GetSlateColor("SelectionColor");
	const FSlateColor SelectionColor_Inactive = FAppStyle::GetSlateColor("SelectionColor_Inactive");
	
	Set("HierarchyEditor.Drop.Section.Left", new CORE_BOX_BRUSH("Common/DropZoneIndicator_Above", FMargin(10.0f / 16.0f, 10.0f / 16.0f, 0.f, 0.f), SelectionColor));
	Set("HierarchyEditor.Drop.Section.Onto", new CORE_BOX_BRUSH("Common/DropZoneIndicator_Onto", FMargin(4.0f / 16.0f), SelectionColor));
	Set("HierarchyEditor.Drop.Section.Right", new CORE_BOX_BRUSH("Common/DropZoneIndicator_Below", FMargin(10.0f / 16.0f, 0.f, 0.f, 10.0f / 16.0f), SelectionColor));
	
	const FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");
	FSlateFontInfo CategoryFont = FAppStyle::Get().GetFontStyle(TEXT("DetailsView.CategoryFontStyle"));
	CategoryFont.Size = 11;
	
	const FEditableTextBoxStyle NormalEditableTextBox = FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
	
	FEditableTextBoxStyle CategoryEditableText = FEditableTextBoxStyle(NormalEditableTextBox)
		.SetFont(CategoryFont)
		.SetForegroundColor(FStyleColors::AccentWhite);
	
	FTextBlockStyle CategoryText = FTextBlockStyle(NormalText)
		.SetFont(CategoryFont);
	FInlineEditableTextBlockStyle HierarchyCategoryTextStyle = FInlineEditableTextBlockStyle()
		.SetTextStyle(CategoryText)
		.SetEditableTextBoxStyle(CategoryEditableText);

	Set("HierarchyEditor.CategoryTextBlock", CategoryText);
	Set("HierarchyEditor.Category", HierarchyCategoryTextStyle);

	FButtonStyle SimpleButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton");
	FButtonStyle ButtonStyle = FButtonStyle(SimpleButtonStyle)
		.SetNormalForeground(FStyleColors::Foreground)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetHoveredForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::Foreground)
		.SetNormalPadding(FMargin(8.f, 2.f, 8.f, 2.f))
		.SetPressedPadding(FMargin(8.f, 3.f, 8.f, 1.f));

	Set("HierarchyEditor.ButtonStyle", ButtonStyle);
	
	FLinearColor HighlightColor = FLinearColor(0.068f, 0.068f, 0.068f);
	
	Set("HierarchyEditor.Color.Highlight", HighlightColor);
	
	FTableRowStyle HierarchyEditorCategoryRowStyle = FTableRowStyle()
		.SetEvenRowBackgroundBrush(*FAppStyle::GetBrush("DetailsView.CategoryTop"))
		.SetOddRowBackgroundBrush(*FAppStyle::GetBrush("DetailsView.CategoryTop"))
		.SetEvenRowBackgroundHoveredBrush(*FAppStyle::GetBrush("DetailsView.CategoryTop_Hovered"))
		.SetOddRowBackgroundHoveredBrush(*FAppStyle::GetBrush("DetailsView.CategoryTop_Hovered"))
		.SetSelectorFocusedBrush( CORE_BORDER_BRUSH( "Common/Selector", FMargin(4.f/16.f), SelectorColor))
		.SetActiveBrush( CORE_IMAGE_BRUSH( "Common/Selection", CoreStyleConstants::Icon8x8, SelectionColor))
		.SetActiveHoveredBrush( CORE_IMAGE_BRUSH( "Common/Selection", CoreStyleConstants::Icon8x8, SelectionColor))
		.SetInactiveBrush( CORE_IMAGE_BRUSH( "Common/Selection", CoreStyleConstants::Icon8x8, SelectionColor_Inactive))
		.SetInactiveHoveredBrush( CORE_IMAGE_BRUSH( "Common/Selection", CoreStyleConstants::Icon8x8, SelectionColor_Inactive))
		.SetActiveHighlightedBrush( CORE_IMAGE_BRUSH("Common/Selection", CoreStyleConstants::Icon8x8, HighlightColor))
		.SetInactiveHighlightedBrush( CORE_IMAGE_BRUSH("Common/Selection", CoreStyleConstants::Icon8x8, HighlightColor))
		.SetDropIndicator_Above(CORE_BOX_BRUSH("Common/DropZoneIndicator_Above", FMargin(10.0f / 16.0f, 10.0f / 16.0f, 0, 0), SelectionColor))
		.SetDropIndicator_Onto(CORE_BOX_BRUSH("Common/DropZoneIndicator_Onto", FMargin(4.0f / 16.0f), SelectionColor))
		.SetDropIndicator_Below(CORE_BOX_BRUSH("Common/DropZoneIndicator_Below", FMargin(10.0f / 16.0f, 0, 0, 10.0f / 16.0f), SelectionColor));
	
	Set("HierarchyEditor.Row.Category", HierarchyEditorCategoryRowStyle);
	
	FInlineEditableTextBlockStyle HierarchyEditorCategoryStyle = FAppStyle::GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockStyle");
	HierarchyEditorCategoryStyle.TextStyle = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DetailsView.CategoryTextStyle");
	HierarchyEditorCategoryStyle.TextStyle.SetFontSize(10.f);
	Set("HierarchyEditor.CategoryTextStyle", HierarchyEditorCategoryStyle);
	
	Set("HierarchyEditor.RootDropIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/caret-down", CoreStyleConstants::Icon20x20));
}

FDataHierarchyEditorStyle::FDataHierarchyEditorStyle() : FSlateStyleSet("DataHierarchyEditorStyle")
{
	InitDataHierarchyEditor();
}

void FDataHierarchyEditorStyle::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const FDataHierarchyEditorStyle& FDataHierarchyEditorStyle::Get()
{
	if(!DataHierarchyEditorStyle.IsValid())
	{
		DataHierarchyEditorStyle = MakeShareable(new FDataHierarchyEditorStyle());
	}
	
	return *DataHierarchyEditorStyle;
}

void FDataHierarchyEditorStyle::ReinitializeStyle()
{
	Unregister();
	DataHierarchyEditorStyle.Reset();
	Register();	
}
