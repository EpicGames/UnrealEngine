// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataHierarchyElementCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FDataHierarchyElementCustomVersion::Guid(0xC1270362, 0xAB230A1F, 0xCB1EC736, 0x71275FAB);

// Register the custom version with core
FCustomVersionRegistration GRegisterDataHierarchyElementCustomVersion(FDataHierarchyElementCustomVersion::Guid, FDataHierarchyElementCustomVersion::LatestVersion, TEXT("DataHierarchyElementVersion"));
