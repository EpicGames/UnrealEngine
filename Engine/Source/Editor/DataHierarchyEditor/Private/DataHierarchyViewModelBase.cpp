// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataHierarchyViewModelBase.h"
#include "SDropTarget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/SDataHierarchyEditor.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Input/SCheckBox.h"
#include "Editor.h"
#include "DataHierarchyEditorCommands.h"
#include "DataHierarchyEditorMisc.h"
#include "DataHierarchyEditorModule.h"
#include "IPropertyRowGenerator.h"
#include "ScopedTransaction.h"
#include "Framework/Commands/GenericCommands.h"
#include "ScopedTransaction.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ToolMenus.h"
#include "Logging/StructuredLog.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"
#include "DataHierarchyElementCustomVersion.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataHierarchyViewModelBase)

#define LOCTEXT_NAMESPACE "DataHierarchyEditor"

FName UDataHierarchyViewModelBase::AllSectionHierarchyObjectName = "AllSection_Hierarchy";
FName UDataHierarchyViewModelBase::AllSectionSourceObjectName = "AllSection_Source";
FText UDataHierarchyViewModelBase::AllSectionDisplayName = LOCTEXT("AllSectionDisplayName", "All");

FText FHierarchyElementViewModel::FDefaultUserFeedback::UneditableTarget = LOCTEXT("DefaultUserFeedback_UneditableTarget", "{0} is not editable.");
FText FHierarchyElementViewModel::FDefaultUserFeedback::TargetCantContainType = LOCTEXT("DefaultUserFeedback_ViewModelCantContainChildType", "{0} can't contain elements of type {1}");
FText FHierarchyElementViewModel::FDefaultUserFeedback::AddElementFromSourceToHierarchy = LOCTEXT("DefaultUserFeedback_AddElementFromSourceToHierarchy", "Add {0} to {1}");
FText FHierarchyElementViewModel::FDefaultUserFeedback::MoveElementWithinHierarchy = LOCTEXT("DefaultUserFeedback_MoveElementWithinHierarchy", "Move {0} to {1}");

UHierarchyElement::UHierarchyElement()
{
	const uint64 DeterministicSeed = 0x377206996352438CLL;

	TStringBuilder<256> PathName;
	GetPathName(nullptr, PathName);
	Identity.Guids.Add(FGuid::NewDeterministicGuid(PathName.ToView(), DeterministicSeed));
}

TConstArrayView<const TObjectPtr<UHierarchyElement>> UHierarchyElement::GetChildren() const
{
	return Children;
}

UHierarchyElement* UHierarchyElement::FindChildWithIdentity(FHierarchyElementIdentity ChildIdentity, bool bSearchRecursively)
{
	TObjectPtr<UHierarchyElement>* FoundItem = Children.FindByPredicate([ChildIdentity](UHierarchyElement* Child)
	{
		return Child->GetPersistentIdentity() == ChildIdentity;
	});

	if(FoundItem)
	{
		return *FoundItem;
	}
	
	if(bSearchRecursively)
	{
		for(UHierarchyElement* Child : Children)
		{
			UHierarchyElement* FoundChild = Child->FindChildWithIdentity(ChildIdentity, bSearchRecursively);

			if(FoundChild)
			{
				return FoundChild;
			}
		}
	}	

	return nullptr;
}

UHierarchyElement* UHierarchyElement::CopyAndAddItemAsChild(const UHierarchyElement& ItemToCopy)
{
	UHierarchyElement* NewChild = Cast<UHierarchyElement>(StaticDuplicateObject(&ItemToCopy, this));
	if(NewChild->GetPersistentIdentity() != ItemToCopy.GetPersistentIdentity())
	{
		check(false);
	}
	GetChildrenMutable().Add(NewChild);

	return NewChild;
}

UHierarchyElement* UHierarchyElement::CopyAndAddItemUnderParentIdentity(const UHierarchyElement& ItemToCopy, FHierarchyElementIdentity ParentIdentity)
{
	UHierarchyElement* ParentItem = FindChildWithIdentity(ParentIdentity, true);

	if(ParentItem)
	{
		UHierarchyElement* NewChild = Cast<UHierarchyElement>(StaticDuplicateObject(&ItemToCopy, ParentItem));
		if(NewChild->GetPersistentIdentity() != ItemToCopy.GetPersistentIdentity())
		{
			check(false);
		}
		ParentItem->GetChildrenMutable().Add(NewChild);
		return NewChild;
	}

	return nullptr;
}

bool UHierarchyElement::RemoveChildWithIdentity(FHierarchyElementIdentity ChildIdentity, bool bSearchRecursively)
{	
	int32 RemovedChildrenCount = Children.RemoveAll([ChildIdentity](UHierarchyElement* Child)
	{
		return Child->GetPersistentIdentity() == ChildIdentity;
	});

	if(RemovedChildrenCount > 1)
	{
		UE_LOGF(LogDataHierarchyEditor, Warning, "More than one child with the same identity has been found in parent %ls", *ToString());
	}
	
	bool bChildrenRemoved = RemovedChildrenCount > 0;
	
	if(bSearchRecursively && bChildrenRemoved == false)
	{
		for(UHierarchyElement* Child : Children)
		{
			bChildrenRemoved |= Child->RemoveChildWithIdentity(ChildIdentity, bSearchRecursively);
		}
	}

	return bChildrenRemoved;
}

TArray<FHierarchyElementIdentity> UHierarchyElement::GetParentIdentities() const
{
	TArray<FHierarchyElementIdentity> ParentIdentities;

	for(UHierarchyElement* Parent = Cast<UHierarchyElement>(GetOuter()); Parent != nullptr; Parent = Cast<UHierarchyElement>(Parent->GetOuter()))
	{
		ParentIdentities.Add(Parent->GetPersistentIdentity());
	}

	return ParentIdentities;
}

TArray<const TObjectPtr<UHierarchyElement>> UHierarchyElement::GetParentElements() const
{
	TArray<const TObjectPtr<UHierarchyElement>> Result;

	for(UHierarchyElement* Parent = Cast<UHierarchyElement>(GetOuter()); Parent != nullptr; Parent = Cast<UHierarchyElement>(Parent->GetOuter()))
	{
		Result.Add(Parent);
	}

	return Result;
}

bool UHierarchyElement::Modify(bool bAlwaysMarkDirty)
{
	bool bSavedToTransactionBuffer = true;

	for(UHierarchyElement* Child : Children)
	{
		bSavedToTransactionBuffer &= Child->Modify(bAlwaysMarkDirty);
	}
	
	bSavedToTransactionBuffer &= UObject::Modify(bAlwaysMarkDirty);

	return bSavedToTransactionBuffer;
}

void UHierarchyElement::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	
	Ar.UsingCustomVersion(FDataHierarchyElementCustomVersion::Guid);
}

void UHierarchyElement::PostLoad()
{
	if(Guid_DEPRECATED.IsValid())
	{
		SetIdentity(FHierarchyElementIdentity({Guid_DEPRECATED}, {}));
	}

	bool bAnyChildNullptr = false;		
	for(auto It = Children.CreateIterator(); It; ++It)
	{
		if(*It == nullptr)
		{
			bAnyChildNullptr = true;
			It.RemoveCurrent();
		}
	}

	if(bAnyChildNullptr)
	{
		UPackage* Package = GetPackage();
		UE_LOGF(LogDataHierarchyEditor, Warning, "HierarchyElement %ls found nullptr child in asset %ls. Removed all nullptr children. This is indicative of something wrong. Check if the hierarchy is still correct and fix it, if necessary.", *ToString(), *GetNameSafe(Package))	
	}
	
	Super::PostLoad();
}

UHierarchySection* UHierarchyRoot::FindSectionByIdentity(FHierarchyElementIdentity SectionIdentity)
{
	for(UHierarchySection* Section : GetSectionDataMutable())
	{
		if(Section->GetPersistentIdentity() == SectionIdentity)
		{
			return Section;
		}
	}

	return nullptr;
}

void UHierarchyRoot::DuplicateSectionFromOtherRoot(const UHierarchySection& SectionToCopy)
{
	if(FindSectionByIdentity(SectionToCopy.GetPersistentIdentity()) != nullptr || SectionToCopy.GetOuter() == this)
	{
		return;
	}
	
	Sections.Add(Cast<UHierarchySection>(StaticDuplicateObject(&SectionToCopy, this)));
}

TSet<FName> UHierarchyRoot::GetSectionNames() const
{
	TSet<FName> OutSections;
	for(const UHierarchySection* Section : Sections)
	{
		OutSections.Add(Section->GetSectionName());
	}

	return OutSections;
}

int32 UHierarchyRoot::GetSectionIndex(UHierarchySection* Section) const
{
	return Sections.Find(Section);
}

bool UHierarchyRoot::Modify(bool bAlwaysMarkDirty)
{
	bool bSavedToTransactionBuffer = true;
	
	for(UHierarchySection* Section : Sections)
	{
		bSavedToTransactionBuffer &= Section->Modify();
	}
	
	bSavedToTransactionBuffer &= Super::Modify(bAlwaysMarkDirty);	

	return bSavedToTransactionBuffer;
}

void UHierarchyRoot::EmptyAllData()
{
	Children.Empty();
	Sections.Empty();
}

void UHierarchyRoot::Serialize(FStructuredArchive::FRecord Record)
{
	// If the root isn't transient, neither should any of its hierarchy elements be.
	// This is expected to happen as the source elements are transient by default.
	// When source hierarchy elements are put into the hierarchy we have to make sure to remove the flag after
	if(Record.GetArchiveState().IsSaving() && this->HasAnyFlags(RF_Transient) == false)
	{
		TArray<UHierarchyElement*> AllElements;
		GetChildrenOfType(AllElements, true);

		for(UHierarchyElement* Element : AllElements)
		{
			Element->ClearFlags(RF_Transient);
		}
	}
	
	Super::Serialize(Record);
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchyCategoryViewModel::CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElement)
{
	return (InHierarchyElement->IsChildOf<UHierarchyItem>() || InHierarchyElement->IsChildOf<UHierarchyCategory>());	
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchyCategoryViewModel::CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedElement, EItemDropZone ItemDropZone)
{
	FResultWithUserFeedback Results(false);
	
	TArray<TSharedPtr<FHierarchyCategoryViewModel>> TargetChildrenCategories;
	GetChildrenViewModelsForType<UHierarchyCategory, FHierarchyCategoryViewModel>(TargetChildrenCategories);
	
	TArray<TSharedPtr<FHierarchyCategoryViewModel>> SiblingCategories;
	Parent.Pin()->GetChildrenViewModelsForType<UHierarchyCategory, FHierarchyCategoryViewModel>(SiblingCategories);
	
	// categories can be dropped on categories, but only if the resulting sibling categories or children categories have different names
	if(DraggedElement->GetData()->IsA<UHierarchyCategory>())
	{
		if(ItemDropZone != EItemDropZone::OntoItem)
		{
			bool bContainsSiblingWithSameName = SiblingCategories.ContainsByPredicate([DraggedElement](TSharedPtr<FHierarchyCategoryViewModel> HierarchyCategoryViewModel)
				{
					return DraggedElement->ToString() == HierarchyCategoryViewModel->ToString() && DraggedElement != HierarchyCategoryViewModel;
				});

			if(bContainsSiblingWithSameName)
			{
				Results.bResult = false;
				Results.UserFeedback = LOCTEXT("CantDropCategorNextToCategorySameSiblingNames", "A category of the same name already exists here, potentially in a different section. Please rename your category first.");
				return Results;
			}

			Results.UserFeedback = LOCTEXT("MoveCategoryText", "Move category here");

			// if we are making a category a sibling of another at the root level, the section will be set to the currently active section. Let that be known.
			if(Parent.Pin()->GetData()->IsA<UHierarchyRoot>())
			{
				UHierarchyCategory* DraggedCategory = Cast<UHierarchyCategory>(DraggedElement->GetDataMutable());
				
				FDataHierarchyElementMetaData_SectionAssociation SectionAssociation = DraggedCategory->FindMetaDataOfTypeOrDefault<FDataHierarchyElementMetaData_SectionAssociation>();					
				if(SectionAssociation.Section != HierarchyViewModel->GetActiveHierarchySectionData())
				{
					FText SectionChangeBaseText = LOCTEXT("CategorySectionWillUpdateDueToDrop", "The section of the category will change to {0} after the drop");
					FText ActualSectionChangeText = FText::FormatOrdered(SectionChangeBaseText, HierarchyViewModel->GetActiveHierarchySectionViewModel()->GetSectionNameAsText());
					Results.UserFeedback = FText::FormatOrdered(FText::AsCultureInvariant("{0}\n{1}"), Results.UserFeedback.Get(FText::GetEmpty()), ActualSectionChangeText);
				}
			}
		}
		else
		{
			bool bContainsChildrenCategoriesWithSameName = TargetChildrenCategories.ContainsByPredicate([DraggedElement](TSharedPtr<FHierarchyCategoryViewModel> HierarchyCategoryViewModel)
				{
					return DraggedElement->ToString() == HierarchyCategoryViewModel->ToString();
				});

			if(bContainsChildrenCategoriesWithSameName)
			{
				Results.bResult = false;
				Results.UserFeedback = LOCTEXT("CantDropCategoryOnCategorySameChildCategoryName", "A sub-category of the same name already exists! Please rename your category first.");
				return Results;
			}

			Results.UserFeedback = LOCTEXT("CreateSubcategory", "Drop category here to create a sub-category");
		}

		Results.bResult = true;
		return Results;
	}
	else if(DraggedElement->GetData()->IsA<UHierarchyItem>())
	{
		// items can generally be dropped onto categories
		Results.bResult = EItemDropZone::OntoItem == ItemDropZone;

		if(Results.bResult)
		{
			if(DraggedElement->IsForHierarchy() == false)
			{
				FText Message = LOCTEXT("AddItemToCategoryDragMessage", "Add {0} to {1}");
				Results.UserFeedback = FText::FormatOrdered(Message, FText::FromString(DraggedElement->ToString()), FText::FromString(ToString()));
			}
			else
			{
				FText Message = LOCTEXT("MoveItemToCategoryDragMessage", "Move {0} to {1}");
				Results.UserFeedback = FText::FormatOrdered(Message, FText::FromString(DraggedElement->ToString()), FText::FromString(ToString()));
			}
		}
	}

	return Results;
}

FHierarchyElementIdentity UHierarchyCategory::ConstructIdentity()
{
	FHierarchyElementIdentity Identity;
	Identity.Names.Add("Category");
	Identity.Guids.Add(FGuid::NewGuid());
	return Identity;
}

void UHierarchyCategory::PostLoad()
{
	Super::PostLoad();
	
	// Some categories were never initialized with a proper identity. We fix this up here.
	if(Identity.IsValid() == false)
	{
		SetIdentity(UHierarchyCategory::ConstructIdentity());
	}
	
	const int32 CustomVer = GetLinkerCustomVersion(FDataHierarchyElementCustomVersion::Guid);
	
	if(CustomVer < FDataHierarchyElementCustomVersion::Type::MetaDataIntroduction)
	{
		if(Section_DEPRECATED)
		{
			FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = FindOrAddMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
			SectionAssociation->Section = Section_DEPRECATED;
			Section_DEPRECATED = nullptr;
		}
	}
}

void UHierarchySection::SetSectionNameAsText(const FText& Text)
{
	Section = FName(Text.ToString());
}

UDataHierarchyViewModelBase::UDataHierarchyViewModelBase()
{
	Commands = MakeShared<FUICommandList>();
}

UDataHierarchyViewModelBase::~UDataHierarchyViewModelBase()
{
	RefreshSourceViewDelegate.Unbind();
	RefreshHierarchyWidgetDelegate.Unbind();
	RefreshSectionsViewDelegate.Unbind();
}

void UDataHierarchyViewModelBase::Initialize()
{		
	bIsFinalized = false;
	
	HierarchyRoot = GetHierarchyRoot();
	HierarchyRoot->SetFlags(RF_Transactional);

	TArray<UHierarchyElement*> AllItems;
	HierarchyRoot->GetChildrenOfType<UHierarchyElement>(AllItems, true);
	for(UHierarchyElement* Item : AllItems)
	{
		Item->SetFlags(RF_Transactional);
	}

	for(UHierarchySection* Section : HierarchyRoot->GetSectionDataMutable())
	{
		Section->SetFlags(RF_Transactional);
	}

	UToolMenus* ToolMenus = UToolMenus::Get();

	FName MenuName = GetContextMenuName();
	if(!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* HierarchyMenu = ToolMenus->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);
		HierarchyMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateStatic(&UDataHierarchyViewModelBase::GenerateDynamicContextMenu));
	}
	
	SetupCommands();

	TSharedPtr<FHierarchyElementViewModel> ViewModel = CreateViewModelForElement(HierarchyRoot, nullptr);
	HierarchyRootViewModel = StaticCastSharedPtr<FHierarchyRootViewModel>(ViewModel); 
	if(!ensureMsgf(HierarchyRootViewModel.IsValid(), TEXT("Make sure that CreateViewModelForData creates a FHierarchyRootViewModel (or derived) for UHierarchyRoot elements")))
	{
		return;
	}
	
	HierarchyRootViewModel->Initialize();
	HierarchyRootViewModel->AddChildFilter(FHierarchyElementViewModel::FOnFilterChild::CreateUObject(this, &UDataHierarchyViewModelBase::FilterForHierarchySection));
	HierarchyRootViewModel->SyncViewModelsToData();
	
	AllSection = NewObject<UHierarchySection>(this, GetSectionDataClass(), AllSectionHierarchyObjectName,  RF_Transient);
	DefaultHierarchySectionViewModel = StaticCastSharedPtr<FHierarchySectionViewModel>(CreateViewModelForElement(AllSection, GetHierarchyRootViewModel().ToSharedRef()));
	DefaultHierarchySectionViewModel->SetSectionName(FName(AllSectionDisplayName.ToString()));
	SetActiveHierarchySection(DefaultHierarchySectionViewModel);
	
	InitializeInternal();

	bIsInitialized = true;
	
	OnInitializedDelegate.ExecuteIfBound();
}

void UDataHierarchyViewModelBase::Finalize()
{	
	HierarchyRootViewModel.Reset();
	HierarchyRoot = nullptr;
	
	FinalizeInternal();
	
	bIsFinalized = true;
	bIsInitialized = false;
}

TSharedPtr<FHierarchyElementViewModel> UDataHierarchyViewModelBase::CreateViewModelForElement(UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> Parent)
{
	// We first give the internal implementation a chance to create view models
	if(TSharedPtr<FHierarchyElementViewModel> CustomViewModel = CreateCustomViewModelForElement(Element, Parent))
	{
		return CustomViewModel;
	}

	// If it wasn't implemented or wasn't covered, we make sure to have default view models
	if(UHierarchyItem* Item = Cast<UHierarchyItem>(Element))
	{
		return MakeShared<FHierarchyItemViewModel>(Item, Parent.ToSharedRef(), this);
	}
	else if(UHierarchyCategory* Category = Cast<UHierarchyCategory>(Element))
	{
		return MakeShared<FHierarchyCategoryViewModel>(Category, Parent.ToSharedRef(), this);
	}
	else if(UHierarchySection* Section = Cast<UHierarchySection>(Element))
	{
		// For sections, we require the parent to be a root view model
		TSharedPtr<FHierarchyRootViewModel> RootViewModel = StaticCastSharedPtr<FHierarchyRootViewModel>(Parent);
		ensure(RootViewModel.IsValid());
		return MakeShared<FHierarchySectionViewModel>(Section, RootViewModel.ToSharedRef(), this);
	}
	else if(UHierarchyRoot* Root = Cast<UHierarchyRoot>(Element))
	{
		// If the root is the hierarchy root, we know it's for the hierarchy. If not, it's the transient source root
		bool bIsForHierarchy = GetHierarchyRoot() == Element;
		return MakeShared<FHierarchyRootViewModel>(Root, this, bIsForHierarchy);
	}

	ensureMsgf(false, TEXT("This should never be reached. Either a custom or a default view model must exist for each Hierarchy Element"));
	return nullptr;
}

void UDataHierarchyViewModelBase::OnHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload)
{
	if(Payload.IsValid())
	{
		if(Payload.GetScriptStruct() == FHierarchyChangedPayload_ElementsAdded::StaticStruct())
		{
			TArray<TSharedPtr<FHierarchyElementViewModel>> AddedElementViewModels = Payload.GetPtr<FHierarchyChangedPayload_ElementsAdded>()->AddedElementViewModels;
			
			bool bContainsHierarchyElement = AddedElementViewModels.ContainsByPredicate([](TSharedPtr<FHierarchyElementViewModel> Candidate)
			{
				return Candidate->GetData()->IsA<UHierarchyItem>() || Candidate->GetData()->IsA<UHierarchyCategory>();
			});
			
			bool bContainsSection = AddedElementViewModels.ContainsByPredicate([](TSharedPtr<FHierarchyElementViewModel> Candidate)
			{
				return Candidate->GetData()->IsA<UHierarchySection>();
			});
			
			// when a new item is created (opposed to dragged & dropped from source view, i.e. only categories so far)
			// we make sure to request a tree refresh, select the row, and request a pending rename since the widget will be created a frame later
			if (bContainsHierarchyElement)
			{
				RefreshHierarchyView(false);
				
				if (AddedElementViewModels.Num() == 1)
				{
					NavigateToElementInHierarchy(AddedElementViewModels[0].ToSharedRef());
				}
			}
			
			if (bContainsSection)
			{
				RefreshSectionsView();
				
				if (AddedElementViewModels.Num() == 1)
				{
					SetActiveHierarchySection(StaticCastSharedPtr<FHierarchySectionViewModel>(AddedElementViewModels[0]));
				}
			}
			
			// For any element added to the root, we fixup section association
			TSharedPtr<FHierarchySectionViewModel> ActiveSectionViewModel = GetActiveHierarchySectionViewModel();
			for (TSharedPtr<FHierarchyElementViewModel> AddedElement : AddedElementViewModels)
			{
				if (AddedElement->GetData()->IsA<UHierarchySection>())
				{
					continue;
				}
				
				TSharedPtr<FHierarchyElementViewModel> ParentViewModel = AddedElement->GetParent().Pin();
				if (!ParentViewModel->GetData()->IsA<UHierarchyRoot>())
				{
					continue;
				}
				
				// If we are in the all section, we clear the section association
				if (ActiveSectionViewModel->IsAllSection())
				{
					AddedElement->GetDataMutable()->DeleteMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
				}
				// If not, we attempt to update the section association
				else
				{
					if (ActiveSectionViewModel->CanContain(AddedElement->GetData()->GetClass()).bResult)
					{
						FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = AddedElement->GetDataMutable()->FindOrAddMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
						SectionAssociation->Section = ActiveSectionViewModel->GetSectionData();
					}
				}
			}

			if (AddedElementViewModels.Num() == 1)
			{
				AddedElementViewModels[0]->RequestRenamePending();
			}
		}

		if(Payload.GetScriptStruct() == FHierarchyChangedPayload_ElementsDeleted::StaticStruct())
		{
			TArray<TSharedPtr<FHierarchyElementViewModel>> DeletedElementViewModels = Payload.GetPtr<FHierarchyChangedPayload_ElementsDeleted>()->DeletedElementViewModels;
			
			bool bContainsHierarchyElement = DeletedElementViewModels.ContainsByPredicate([](TSharedPtr<FHierarchyElementViewModel> Candidate)
			{
				return Candidate->GetData()->IsA<UHierarchyItem>() || Candidate->GetData()->IsA<UHierarchyCategory>();
			});
			
			bool bContainsSection = DeletedElementViewModels.ContainsByPredicate([](TSharedPtr<FHierarchyElementViewModel> Candidate)
			{
				return Candidate->GetData()->IsA<UHierarchySection>();
			});
			
			bool bContainsActiveSection = DeletedElementViewModels.Contains(ActiveHierarchySectionViewModel.Pin());
			
			
			if(bContainsHierarchyElement)
			{
				RefreshHierarchyView(false);
			}
			else if(bContainsSection)
			{
				RefreshSectionsView();

				if(bContainsActiveSection)
				{
					SetActiveHierarchySection(GetDefaultHierarchySectionViewModel());
				}
			}
		}
	}
	
	OnHierarchyChangedDelegate.Broadcast(Payload);
}

TSubclassOf<UHierarchyCategory> UDataHierarchyViewModelBase::GetCategoryDataClass() const
{
	return UHierarchyCategory::StaticClass();
}

TSubclassOf<UHierarchySection> UDataHierarchyViewModelBase::GetSectionDataClass() const
{
	return UHierarchySection::StaticClass();
}

void UDataHierarchyViewModelBase::ForceFullRefresh()
{
	RefreshSourceItemsRequestedDelegate.ExecuteIfBound();
	// todo (me) during merge at startup this can be nullptr for some reason
	if(HierarchyRootViewModel.IsValid())
	{
		HierarchyRootViewModel->SyncViewModelsToData();
	}
	RefreshAllViewsRequestedDelegate.ExecuteIfBound(true);
}

void UDataHierarchyViewModelBase::ForceFullRefreshOnTimer()
{
	ensure(FullRefreshNextFrameHandle.IsValid());
	ForceFullRefresh();
	FullRefreshNextFrameHandle.Invalidate();
}

void UDataHierarchyViewModelBase::RequestFullRefreshNextFrame()
{
	if(!FullRefreshNextFrameHandle.IsValid() && GEditor != nullptr && GEditor->IsTimerManagerValid())
	{
		FullRefreshNextFrameHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateUObject(this, &UDataHierarchyViewModelBase::ForceFullRefreshOnTimer));
	}
}

void UDataHierarchyViewModelBase::RefreshAllViews(bool bFullRefresh) const
{
	RefreshAllViewsRequestedDelegate.ExecuteIfBound(bFullRefresh);
}

void UDataHierarchyViewModelBase::RefreshSourceView(bool bFullRefresh) const
{
	RefreshSourceViewDelegate.ExecuteIfBound(bFullRefresh);
}

void UDataHierarchyViewModelBase::RefreshHierarchyView(bool bFullRefresh) const
{
	RefreshHierarchyWidgetDelegate.ExecuteIfBound(bFullRefresh);
}

void UDataHierarchyViewModelBase::RefreshSectionsView() const
{
	RefreshSectionsViewDelegate.ExecuteIfBound();
}

void UDataHierarchyViewModelBase::PostUndo(bool bSuccess)
{
	ForceFullRefresh();
}

void UDataHierarchyViewModelBase::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

bool UDataHierarchyViewModelBase::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	for(const TPair<UObject*, FTransactionObjectEvent>& TransactionObjectContext : TransactionObjectContexts)
	{
		if(TransactionObjectContext.Key->IsA<UHierarchyElement>())
		{
			return true;
		}
	}
	
	return false;
}

bool UDataHierarchyViewModelBase::FilterForHierarchySection(TSharedPtr<const FHierarchyElementViewModel> ViewModel) const
{
	if(ActiveHierarchySectionViewModel.IsValid())
	{
		// If the currently selected section data is nullptr, it's the All section, and we let everything pass
		if(ActiveHierarchySectionViewModel.Pin()->IsAllSection())
		{
			return true;
		}

		// if not, we check against identical section data
		FDataHierarchyElementMetaData_SectionAssociation SectionAssociation = ViewModel->GetData()->FindMetaDataOfTypeOrDefault<FDataHierarchyElementMetaData_SectionAssociation>();		
		return ActiveHierarchySectionViewModel.Pin()->GetSectionData() == SectionAssociation.Section;
	}

	return true;
}

void UDataHierarchyViewModelBase::ToolMenuRequestRename(const FToolMenuContext& Context) const
{
	UHierarchyMenuContext* HierarchyMenuContext = Context.FindContext<UHierarchyMenuContext>();
	if(HierarchyMenuContext->MenuHierarchyElements.Num() == 1)
	{
		HierarchyMenuContext->MenuHierarchyElements[0]->RequestRename();
	}
}

bool UDataHierarchyViewModelBase::ToolMenuCanRequestRename(const FToolMenuContext& Context) const
{
	UHierarchyMenuContext* HierarchyMenuContext = Context.FindContext<UHierarchyMenuContext>();
	if(HierarchyMenuContext->MenuHierarchyElements.Num() == 1)
	{
		return HierarchyMenuContext->MenuHierarchyElements[0]->CanRename();
	}

	return false;
}

void UDataHierarchyViewModelBase::ToolMenuDelete(const FToolMenuContext& Context)
{
	UHierarchyMenuContext* HierarchyMenuContext = Context.FindContext<UHierarchyMenuContext>();
	DeleteElements(HierarchyMenuContext->MenuHierarchyElements);
}

bool UDataHierarchyViewModelBase::ToolMenuCanDelete(const FToolMenuContext& Context) const
{
	UHierarchyMenuContext* HierarchyMenuContext = Context.FindContext<UHierarchyMenuContext>();
	
	for(TSharedPtr<FHierarchyElementViewModel> MenuHierarchyElement : HierarchyMenuContext->MenuHierarchyElements)
	{
		if(MenuHierarchyElement->CanDelete() == false)
		{
			return false;
		}
	}

	return HierarchyMenuContext->MenuHierarchyElements.Num() > 0;
}

void UDataHierarchyViewModelBase::ToolMenuNavigateTo(const FToolMenuContext& Context) const
{
	UHierarchyMenuContext* HierarchyMenuContext = Context.FindContext<UHierarchyMenuContext>();

	if(HierarchyMenuContext->MenuHierarchyElements.Num() == 1)
	{
		if(TSharedPtr<FHierarchyElementViewModel> MatchingViewModelInHierarchy = GetHierarchyRootViewModel()->FindViewModelForChild(HierarchyMenuContext->MenuHierarchyElements[0]->GetData()->GetPersistentIdentity(), true))
		{
			NavigateToElementInHierarchy(MatchingViewModelInHierarchy.ToSharedRef());
		}
	}
}

bool UDataHierarchyViewModelBase::ToolMenuCanNavigateTo(const FToolMenuContext& Context) const
{
	UHierarchyMenuContext* HierarchyMenuContext = Context.FindContext<UHierarchyMenuContext>();

	if(HierarchyMenuContext->MenuHierarchyElements.Num() != 1)
	{
		return false;
	}
	
	TSharedPtr<FHierarchyElementViewModel> ViewModel = HierarchyMenuContext->MenuHierarchyElements[0];
	if(ViewModel->IsForHierarchy())
	{
		return false;
	}
	
	if(TSharedPtr<FHierarchyElementViewModel> MatchingViewModelInHierarchy = GetHierarchyRootViewModel()->FindViewModelForChild(ViewModel->GetData()->GetPersistentIdentity(), true))
	{
		return MatchingViewModelInHierarchy.IsValid();
	}
	
	return false;
}

const TArray<TSharedPtr<FHierarchyElementViewModel>>& UDataHierarchyViewModelBase::GetHierarchyItems() const
{
	return HierarchyRootViewModel->GetFilteredChildren();
}

FName UDataHierarchyViewModelBase::GetContextMenuName() const
{
	return FName(FString(TEXT("HierarchyEditor.") + GetClass()->GetName()));
}

TSharedRef<FHierarchyDragDropOp> UDataHierarchyViewModelBase::CreateDragDropOp(const FDataHierarchyDragDropContext& DragContext)
{
	TSharedRef<FHierarchyDragDropOp> DragDropOp = MakeShared<FHierarchyDragDropOp>(DragContext);
	DragDropOp->Construct();
	return DragDropOp;
}

void UDataHierarchyViewModelBase::OnGetChildren(TSharedPtr<FHierarchyElementViewModel> Element, TArray<TSharedPtr<FHierarchyElementViewModel>>& OutChildren) const
{
	OutChildren.Append(Element->GetFilteredChildren());
}

void UDataHierarchyViewModelBase::SetActiveHierarchySection(TSharedPtr<FHierarchySectionViewModel> Section)
{
	ActiveHierarchySectionViewModel = Section;	
	RefreshHierarchyView(true);
	OnHierarchySectionActivatedDelegate.ExecuteIfBound(Section);
}

TSharedPtr<FHierarchySectionViewModel> UDataHierarchyViewModelBase::GetActiveHierarchySectionViewModel() const
{
	return ActiveHierarchySectionViewModel.Pin();
}

const UHierarchySection* UDataHierarchyViewModelBase::GetActiveHierarchySectionData() const
{
	return ActiveHierarchySectionViewModel.IsValid() ? ActiveHierarchySectionViewModel.Pin()->GetSectionData() : nullptr;
}

bool UDataHierarchyViewModelBase::IsHierarchySectionActive(const UHierarchySection* Section) const
{
	return ActiveHierarchySectionViewModel.IsValid() ? ActiveHierarchySectionViewModel.Pin()->GetSectionData() == Section : false;
}

FString UDataHierarchyViewModelBase::OnElementToStringDebug(TSharedPtr<FHierarchyElementViewModel> ElementViewModel) const
{
	return ElementViewModel->ToString();	
}

FHierarchyElementViewModel::~FHierarchyElementViewModel()
{
	Children.Empty();
	FilteredChildren.Empty();
}

TSharedPtr<FHierarchyElementViewModel> FHierarchyElementViewModel::AddChild(TSubclassOf<UHierarchyElement> NewChildClass, int32 InsertIndex, FHierarchyElementIdentity ChildIdentity)
{
	UHierarchyElement* NewChild = GetDataMutable()->AddChild<UHierarchyElement>(NewChildClass, InsertIndex);
	NewChild->SetFlags(RF_Transactional);
	NewChild->SetIdentity(ChildIdentity);
	NewChild->Modify();
	
	SyncViewModelsToData();

	TSharedPtr<FHierarchyElementViewModel> ElementViewModel = FindViewModelForChild(NewChild);
	ensureMsgf(ElementViewModel.IsValid(), TEXT("Invalid view model detected after syncing view models to data."));
	
	TInstancedStruct<FHierarchyChangedPayload_ElementsAdded> Payload = TInstancedStruct<FHierarchyChangedPayload_ElementsAdded>::Make();
	Payload.GetMutable<FHierarchyChangedPayload_ElementsAdded>().AddedElementViewModels.Add(ElementViewModel);
	HierarchyViewModel->OnHierarchyChanged(Payload);
	
	return ElementViewModel;
}

void FHierarchyElementViewModel::Tick(float DeltaTime)
{
	if(bRenamePending)
	{
		RequestRename();
	}
}

TStatId FHierarchyElementViewModel::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FHierarchyElementViewModel, STATGROUP_Tickables);
}

void FHierarchyElementViewModel::RefreshChildrenData()
{
	TArray<TSharedPtr<FHierarchyElementViewModel>> TmpChildren = Children;
	for(TSharedPtr<FHierarchyElementViewModel> Child : TmpChildren)
	{
		if(Child->RepresentsExternalData() && Child->DoesExternalDataStillExist(HierarchyViewModel->GetRefreshContext()) == false)
		{
			UE_LOGFMT(LogDataHierarchyEditor, Verbose, "Hierarchy Element {ElementName} no longer has valid external data. Deleting.", Child->ToString());
			Child->Delete();
		}
	}

	/** Every item view model can define its own sort order for its children. */
	SortChildrenData();
	
	RefreshChildrenDataInternal();

	/** All remaining children are supposed to exist at this point, as internal data won't be removed by refreshing & external data was cleaned up already.
	 * This will not call RefreshChildrenData on data that has just been added as no view models exist for these yet.
	 */
	for(TSharedPtr<FHierarchyElementViewModel> Child : Children)
	{
		Child->RefreshChildrenData();
	}
}

void FHierarchyElementViewModel::SyncViewModelsToData()
{
	// Guard against re-entry on the same element. A listener on OnHierarchyChangedDelegate can respond to a mutation that
	// fires mid-walk (e.g. a Child->Delete() in RefreshChildrenData) by synchronously calling SyncViewModelsToData on the
	// root again. Without this guard that inner call rebuilds Children while outer frames are still iterating it. The flag
	// is per-element, so the legitimate recursive descent into child view models is unaffected.
	if (bIsSyncInProgress)
	{
		UE_LOGFMT(LogDataHierarchyEditor, Warning, "Re-entrant SyncViewModelsToData call on element {ElementName} skipped. The outer sync in progress will produce the final state. Did you call SyncViewModelsToData in the middle of another sync (for example from an OnHierarchyChanged listener)? Set LogDataHierarchyEditor to Verbose to dump the callstack and help find the callsite.", ToString());
		if (UE_LOG_ACTIVE(LogDataHierarchyEditor, Verbose))
		{
			FDebug::DumpStackTraceToLog(TEXT("Re-entrant SyncViewModelsToData callstack:"), ELogVerbosity::Log);
		}
		return;
	}
	TGuardValue<bool> SyncGuard(bIsSyncInProgress, true);

	// this will recursively remove all outdated external data as well as give individual view models the chance to add new data
	RefreshChildrenData();
	
	// now that the data is refreshed, we can sync to the data by recycling view models & creating new ones
	// old view models will get deleted automatically
	TArray<TSharedPtr<FHierarchyElementViewModel>> NewChildren;
	for(UHierarchyElement* Child : Element->GetChildren())
	{		
		int32 ViewModelIndex = FindIndexOfChild(Child);
		// if we couldn't find a view model for a data child, we create it here
		if(ViewModelIndex == INDEX_NONE)
		{
			TSharedPtr<FHierarchyElementViewModel> ChildViewModel = HierarchyViewModel->CreateViewModelForElement(Child, AsShared());
			if(ensure(ChildViewModel.IsValid()))
			{
				ChildViewModel->Initialize();
				ChildViewModel->SyncViewModelsToData();
				NewChildren.Add(ChildViewModel);
			}
		}
		// if we could find the view model, we refresh its contained view models and readd it
		else
		{
			Children[ViewModelIndex]->SyncViewModelsToData();
			NewChildren.Add(Children[ViewModelIndex]);
		}
	}

	Children.Empty();
	Children.Append(NewChildren);
	
	for(TSharedPtr<FHierarchyElementViewModel> Child : Children)
	{
		Child->OnChildRequestedDeletion().BindSP(this, &FHierarchyElementViewModel::DeleteChild);
		Child->GetOnSynced().BindSP(this, &FHierarchyElementViewModel::PropagateOnChildSynced);
	}

	/** Give the view models a chance to further customize the children sync process. */
	SyncViewModelsToDataInternal();	

	// then we sort the view models according to the data order as this is what will determine widget order created from the view models
	Children.Sort([this](const TSharedPtr<FHierarchyElementViewModel>& ItemA, const TSharedPtr<FHierarchyElementViewModel>& ItemB)
		{
			return FindIndexOfDataChild(ItemA) < FindIndexOfDataChild(ItemB);
		});
	
	// we refresh the filtered children here as well
	GetFilteredChildren();

	OnSyncedDelegate.ExecuteIfBound();
}

const TArray<TSharedPtr<FHierarchyElementViewModel>>& FHierarchyElementViewModel::GetFilteredChildren() const
{
	FilteredChildren.Empty();

	if(CanHaveChildren().bResult)
	{
		for(TSharedPtr<FHierarchyElementViewModel> Child : Children)
		{
			bool bPassesFilter = true;
			for(const FOnFilterChild& OnFilterChild : ChildFilters)
			{
				bPassesFilter &= OnFilterChild.Execute(Child);

				if(!bPassesFilter)
				{
					break;
				}
			}

			if(bPassesFilter)
			{
				FilteredChildren.Add(Child);
			}
		}
	}

	return FilteredChildren;
}

void FHierarchyElementViewModel::SortChildrenData()
{
	GetDataMutable()->GetChildrenMutable().StableSort([](const UHierarchyElement& ItemA, const UHierarchyElement& ItemB)
	{
		return ItemA.IsA<UHierarchyCategory>() && ItemB.IsA<UHierarchyItem>();
	});
}

int32 FHierarchyElementViewModel::GetHierarchyDepth() const
{
	if(Parent.IsValid())
	{
		return 1 + Parent.Pin()->GetHierarchyDepth();
	}

	return 0;
}

void FHierarchyElementViewModel::AddChildFilter(FOnFilterChild InFilterChild)
{
	if(ensure(InFilterChild.IsBound()))
	{
		ChildFilters.Add(InFilterChild);
	}
}

bool FHierarchyElementViewModel::HasParent(TSharedPtr<FHierarchyElementViewModel> ParentCandidate, bool bRecursive) const
{
	if(Parent.IsValid())
	{
		if(Parent == ParentCandidate)
		{
			return true;
		}
		else if(bRecursive)
		{
			return Parent.Pin()->HasParent(ParentCandidate, bRecursive);
		}
	}

	return false;
}

TSharedRef<FHierarchyElementViewModel> FHierarchyElementViewModel::DuplicateToThis(TSharedPtr<FHierarchyElementViewModel> ItemToDuplicate, int32 InsertIndex)
{
	UHierarchyElement* NewItem = Cast<UHierarchyElement>(StaticDuplicateObject(ItemToDuplicate->GetData(), GetDataMutable()));
	if(InsertIndex == INDEX_NONE)
	{
		GetDataMutable()->GetChildrenMutable().Add(NewItem);
	}
	else
	{
		GetDataMutable()->GetChildrenMutable().Insert(NewItem, InsertIndex);
	}
	
	SyncViewModelsToData();
	
	TSharedPtr<FHierarchyElementViewModel> ViewModel = FindViewModelForChild(NewItem);
	
	TInstancedStruct<FHierarchyChangedPayload_ElementsAdded> Payload = TInstancedStruct<FHierarchyChangedPayload_ElementsAdded>::Make();
	Payload.GetMutablePtr<FHierarchyChangedPayload_ElementsAdded>()->AddedElementViewModels.Add(ViewModel);
	HierarchyViewModel->OnHierarchyChanged(Payload);
	
	return ViewModel.ToSharedRef();
}

TSharedRef<FHierarchyElementViewModel> FHierarchyElementViewModel::ReparentToThis(TSharedPtr<FHierarchyElementViewModel> ItemToMove, int32 InsertIndex)
{
	UHierarchyElement* NewItem = Cast<UHierarchyElement>(StaticDuplicateObject(ItemToMove->GetData(), GetDataMutable()));
	if(InsertIndex == INDEX_NONE)
	{
		GetDataMutable()->GetChildrenMutable().Add(NewItem);
	}
	else
	{
		GetDataMutable()->GetChildrenMutable().Insert(NewItem, InsertIndex);
	}
	
	ItemToMove->Delete();
	
	SyncViewModelsToData();
	
	TSharedPtr<FHierarchyElementViewModel> ViewModel = FindViewModelForChild(NewItem);
	
	TInstancedStruct<FHierarchyChangedPayload_ElementsMoved> Payload = TInstancedStruct<FHierarchyChangedPayload_ElementsMoved>::Make();
	Payload.GetMutablePtr<FHierarchyChangedPayload_ElementsMoved>()->MovedElementViewModels.Add(ViewModel);
	HierarchyViewModel->OnHierarchyChanged(Payload);
	
	return ViewModel.ToSharedRef();
}

TSharedPtr<FHierarchyElementViewModel> FHierarchyElementViewModel::FindViewModelForChild(const UHierarchyElement* Child, bool bSearchRecursively) const
{
	int32 Index = FindIndexOfChild(Child);
	if(Index != INDEX_NONE)
	{
		return Children[Index];
	}

	if(bSearchRecursively)
	{
		for(TSharedPtr<FHierarchyElementViewModel> ChildViewModel : Children)
		{
			TSharedPtr<FHierarchyElementViewModel> FoundViewModel = ChildViewModel->FindViewModelForChild(Child, bSearchRecursively);

			if(FoundViewModel.IsValid())
			{
				return FoundViewModel;
			}
		}
	}

	return nullptr;
}

TSharedPtr<FHierarchyElementViewModel> FHierarchyElementViewModel::FindViewModelForChild(FHierarchyElementIdentity ChildIdentity, bool bSearchRecursively) const
{
	for(TSharedPtr<FHierarchyElementViewModel> Child : Children)
	{
		if(Child->GetData()->GetPersistentIdentity() == ChildIdentity)
		{
			return Child;
		}
	}

	if(bSearchRecursively)
	{
		for(TSharedPtr<FHierarchyElementViewModel> ChildViewModel : Children)
		{
			TSharedPtr<FHierarchyElementViewModel> FoundViewModel = ChildViewModel->FindViewModelForChild(ChildIdentity, bSearchRecursively);

			if(FoundViewModel.IsValid())
			{
				return FoundViewModel;
			}
		}
	}

	return nullptr;
}

TSharedPtr<FHierarchyElementViewModel> FHierarchyElementViewModel::FindViewModelForObject(const UObject* Object, bool bSearchRecursively) const
{
	// If it's a hierarchy element we use the dedicated function
	if (Object->IsA<UHierarchyElement>())
	{
		return FindViewModelForChild(Cast<UHierarchyElement>(Object), bSearchRecursively);
	}
	
	// If not, we are looking for the view model that points to the object for editing
	for (TSharedPtr<FHierarchyElementViewModel> ChildViewModel : Children)
	{
		if (UObject* ObjectForEditing = ChildViewModel->GetObjectForEditing())
		{
			if (ObjectForEditing == Object)
			{
				return ChildViewModel;
			}
		}
	}
	
	if (bSearchRecursively)
	{
		for(TSharedPtr<FHierarchyElementViewModel> ChildViewModel : Children)
		{
			TSharedPtr<FHierarchyElementViewModel> FoundViewModel = ChildViewModel->FindViewModelForObject(Object, bSearchRecursively);

			if(FoundViewModel.IsValid())
			{
				return FoundViewModel;
			}
		}
	}
	
	return nullptr;
}

int32 FHierarchyElementViewModel::FindIndexOfChild(const UHierarchyElement* Child) const
{
	return Children.FindLastByPredicate([Child](TSharedPtr<FHierarchyElementViewModel> Item)
	{
		return Item->GetData() == Child;
	});
}

int32 FHierarchyElementViewModel::FindIndexOfDataChild(TSharedPtr<const FHierarchyElementViewModel> Child) const
{
	return GetData()->GetChildren().IndexOfByKey(Child->GetData());
}

int32 FHierarchyElementViewModel::FindIndexOfDataChild(const UHierarchyElement* Child) const
{
	return GetData()->GetChildren().IndexOfByKey(Child);
}

void FHierarchyElementViewModel::Delete()
{
	OnChildRequestedDeletionDelegate.Execute(AsShared());
}

void FHierarchyElementViewModel::DeleteChild(TSharedPtr<FHierarchyElementViewModel> Child)
{	
	ensure(Child->GetParent().Pin() == AsShared());
	GetDataMutable()->Modify();
	GetDataMutable()->GetChildrenMutable().Remove(Child->GetDataMutable());
	Children.Remove(Child);

	TInstancedStruct<FHierarchyChangedPayload_ElementsDeleted> Payload = TInstancedStruct<FHierarchyChangedPayload_ElementsDeleted>::Make();
	Payload.GetMutable<FHierarchyChangedPayload_ElementsDeleted>().DeletedElementViewModels.Add(Child);
	HierarchyViewModel->OnHierarchyChanged(Payload);
}

TOptional<EItemDropZone> FHierarchyElementViewModel::OnCanRowAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone, TSharedPtr<FHierarchyElementViewModel> Item)
{
	if(TSharedPtr<FHierarchyDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FHierarchyDragDropOp>())
	{
		TArray<FDraggedElementEntry> TopLevelEntries = DragDropOp->GetDragContext().GetTopLevelEntries();

		// Single-drop.
		if (TopLevelEntries.Num() == 1)
		{
			TSharedPtr<FHierarchyElementViewModel> DraggedElement = TopLevelEntries[0].Element.Pin();
			if (!DraggedElement.IsValid())
			{
				return TOptional<EItemDropZone>();
			}

			FResultWithUserFeedback Result = CanDropOn(DraggedElement, ItemDropZone);
			if (Result.UserFeedback.IsSet())
			{
				DragDropOp->SetDescription(Result.UserFeedback.GetValue());
			}

			return Result.bResult ? TOptional<EItemDropZone>(ItemDropZone) : TOptional<EItemDropZone>();
		}

		// Multi-drop.
		if (TopLevelEntries.Num() > 1)
		{
			int32 SuccessCount = 0;
			int32 TotalCount = 0;
			FText RejectionMessage;
			for (const FDraggedElementEntry& Entry : TopLevelEntries)
			{
				TSharedPtr<FHierarchyElementViewModel> DraggedElement = Entry.Element.Pin();
				if (!DraggedElement.IsValid())
				{
					continue;
				}

				++TotalCount;
				FResultWithUserFeedback Result = CanDropOn(DraggedElement, ItemDropZone);
				if (Result.bResult)
				{
					++SuccessCount;
				}
				else if (RejectionMessage.IsEmpty() && Result.UserFeedback.IsSet())
				{
					RejectionMessage = Result.UserFeedback.GetValue();
				}
			}

			if (SuccessCount == 0)
			{
				if (RejectionMessage.IsEmpty() == false)
				{
					DragDropOp->SetDescription(RejectionMessage);
				}
				return TOptional<EItemDropZone>();
			}

			const bool bAddingItems = TopLevelEntries[0].Element.IsValid() && (TopLevelEntries[0].Element.Pin()->IsForHierarchy() == false);
			const FText TargetText = this->ToStringAsText();
			FText MultiDropDescription;
			if (SuccessCount == TotalCount)
			{
				const FText FormatText = bAddingItems ? LOCTEXT("MultiDropAdd", "Add {0} items to {1}") : LOCTEXT("MultiDropMove", "Move {0} items to {1}");
				MultiDropDescription = FText::Format(FormatText, TotalCount, TargetText);
			}
			else
			{
				const FText FormatText = bAddingItems ? LOCTEXT("MultiDropAddPartial", "Add {0} of {1} items to {2}") : LOCTEXT("MultiDropMovePartial", "Move {0} of {1} items to {2}");
				MultiDropDescription = FText::Format(FormatText, SuccessCount, TotalCount, TargetText);
			}

			DragDropOp->SetDescription(MultiDropDescription);

			return ItemDropZone;
		}
	}

	return TOptional<EItemDropZone>();
}

FReply FHierarchyElementViewModel::OnDroppedOnRow(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone, TSharedPtr<FHierarchyElementViewModel> Item)
{
	if(TSharedPtr<FHierarchyDragDropOp> HierarchyDragDropOp = DragDropEvent.GetOperationAs<FHierarchyDragDropOp>())
	{
		TArray<FDraggedElementEntry> TopLevelEntries = HierarchyDragDropOp->GetDragContext().GetTopLevelEntries();

		FScopedTransaction Transaction(LOCTEXT("Transaction_MultiDrop", "Dropped items in the hierarchy"));
		HierarchyViewModel->GetHierarchyRoot()->Modify();

		for(const FDraggedElementEntry& Entry : TopLevelEntries)
		{
			TSharedPtr<FHierarchyElementViewModel> DroppedElement = Entry.Element.Pin();
			if (DroppedElement.IsValid() && CanDropOn(DroppedElement, ItemDropZone).bResult)
			{
				OnDroppedOn(DroppedElement, ItemDropZone);
			}
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void FHierarchyElementViewModel::OnRowDragLeave(const FDragDropEvent& DragDropEvent)
{
	if(TSharedPtr<FHierarchyDragDropOp> HierarchyDragDropOp = DragDropEvent.GetOperationAs<FHierarchyDragDropOp>())
	{
		HierarchyDragDropOp->SetDescription(FText::GetEmpty());
	}
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchyElementViewModel::CanDrag()
{
	FResultWithUserFeedback Results = IsEditableByUser();
	if(Results.bResult == false)
	{
		return Results;
	}

	return CanDragInternal();
}

void FHierarchyElementViewModel::Rename(FName NewName)
{
	FName OldName = FName(ToString());
	
	RenameInternal(NewName); 
	
	TInstancedStruct<FHierarchyChangedPayload_ElementNameChanged> Payload = TInstancedStruct<FHierarchyChangedPayload_ElementNameChanged>::Make();
	Payload.GetMutable<FHierarchyChangedPayload_ElementNameChanged>().ElementViewModel = AsShared();
	Payload.GetMutable<FHierarchyChangedPayload_ElementNameChanged>().OldName = OldName;
	Payload.GetMutable<FHierarchyChangedPayload_ElementNameChanged>().NewName = NewName;
	
	HierarchyViewModel->OnHierarchyChanged(Payload);
}

bool FHierarchyElementViewModel::IsExpandedByDefault() const
{
	// Otherwise, let the element decide
	return IsExpandedByDefaultInternal();
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchyElementViewModel::CanDropOn(TSharedPtr<FHierarchyElementViewModel> DraggedElement, EItemDropZone ItemDropZone)
{
	TOptional<FResultWithUserFeedback> CanDropOnOverrideResult = CanDropOnOverride(DraggedElement, ItemDropZone);
	if(CanDropOnOverrideResult.IsSet())
	{
		return CanDropOnOverrideResult.GetValue();
	}
	
	// Editability check
	{
		FResultWithUserFeedback Results = IsEditableByUser();
		if(Results.bResult == false)
		{
			if(Results.UserFeedback.IsSet() == false)
			{
				Results.UserFeedback = FText::FormatOrdered(FDefaultUserFeedback::UneditableTarget, this->ToStringAsText());
			}
			
			return Results;
		}
	}

	// Self identity check
	if(DraggedElement->GetData() == GetData())
	{
		return false;
	}

	// Immediate parent check; refuse if we are already parented to the target
	if(DraggedElement->HasParent(AsShared(), false) && ItemDropZone == EItemDropZone::OntoItem)
	{
		return false;
	}

	// Recursive child check; refuse if we try to drag a parent onto its child
	if(HasParent(DraggedElement, true))
	{
		return false;
	}
	
	// Can Contain checks
	if(ItemDropZone == EItemDropZone::OntoItem)
	{
		FResultWithUserFeedback Results = CanContain(DraggedElement->GetData()->GetClass());
		if(Results.bResult == false)
		{
			if(Results.UserFeedback.IsSet() == false)
			{
				Results.UserFeedback = FText::FormatOrdered(FDefaultUserFeedback::TargetCantContainType,
					GetData()->GetClass()->GetDisplayNameText(), DraggedElement->GetData()->GetClass()->GetDisplayNameText());
			}
			
			return Results;
		}
	}
	else
	{
		if(Parent.IsValid())
		{
			FResultWithUserFeedback Results = Parent.Pin()->CanContain(DraggedElement->GetData()->GetClass());
			if(Results.bResult == false)
			{
				if(Results.UserFeedback.IsSet() == false)
				{
					Results.UserFeedback = FText::FormatOrdered(FDefaultUserFeedback::TargetCantContainType,
						GetData()->GetClass()->GetDisplayNameText(), Parent.Pin()->GetData()->GetClass()->GetDisplayNameText());
				}
				
				return Results;
			}
		}
	}

	// Internal check
	FResultWithUserFeedback Results = CanDropOnInternal(DraggedElement, ItemDropZone);
	
	if(Results.bResult && Results.UserFeedback.IsSet() == false)
	{
		if(DraggedElement->IsForHierarchy() == false)
		{
			Results.UserFeedback = FText::FormatOrdered(FDefaultUserFeedback::AddElementFromSourceToHierarchy, DraggedElement->ToStringAsText(), this->ToStringAsText());
		}
		else
		{
			Results.UserFeedback = FText::FormatOrdered(FDefaultUserFeedback::MoveElementWithinHierarchy, DraggedElement->ToStringAsText(), this->ToStringAsText());
		}
	}
	
	return Results;
}

void FHierarchyElementViewModel::OnDroppedOn(TSharedPtr<FHierarchyElementViewModel> DroppedItem, EItemDropZone ItemDropZone)
{
	if(OnDroppedOnOverride(DroppedItem, ItemDropZone).IsSet())
	{
		return;
	}
	
	FScopedTransaction Transaction(LOCTEXT("Transaction_MovedItem", "Moved an item in the hierarchy"));
	HierarchyViewModel->GetHierarchyRoot()->Modify();
	HierarchyViewModel->GetHierarchyRootViewModel()->PreEditHierarchyStructure();

	// We calculate the target view model (this if OntoItem, Parent if Above/Below the current item)
	// and the insert index (INDEX_NONE means adding at the end)
	TSharedPtr<FHierarchyElementViewModel> TargetViewModel;
	int32 InsertIndex = INDEX_NONE;
	if(ItemDropZone == EItemDropZone::OntoItem)
	{
		TargetViewModel = AsShared();
		InsertIndex = INDEX_NONE;
	}
	else
	{
		if(Parent.IsValid())
		{
			TargetViewModel =  Parent.Pin();
			int32 IndexOfThis = Parent.Pin()->FindIndexOfDataChild(AsShared());

			// If above an item, we insert at this item's index
			if(ItemDropZone == EItemDropZone::AboveItem)
			{
				InsertIndex = FMath::Max(IndexOfThis, 0);
			}
			// If below an item, we insert one index behind this item's index
			else
			{
				InsertIndex = FMath::Min(IndexOfThis+1, Parent.Pin()->GetChildren().Num());
			}
		}
	}

	TSharedPtr<FHierarchyElementViewModel> NewItemViewModel;
	if(ensure(TargetViewModel.IsValid()))
	{
		// If the dropped item is from the source, we duplicate it over
		if(DroppedItem->IsForHierarchy() == false)
		{
			NewItemViewModel = TargetViewModel->DuplicateToThis(DroppedItem, InsertIndex);
		}
		// If the dropped item is from the hierarchy, we simply move it over
		else
		{
			NewItemViewModel = TargetViewModel->ReparentToThis(DroppedItem, InsertIndex);
		}
	}
	
	if(NewItemViewModel.IsValid())
	{
		HierarchyViewModel->GetHierarchyRootViewModel()->PostEditHierarchyStructure();
		NewItemViewModel->PostAddFixup();

		TargetViewModel->PostOnDroppedOn(NewItemViewModel);
		
		HierarchyViewModel->RefreshHierarchyView();
		HierarchyViewModel->RefreshSourceView();
	}
	else
	{
		Transaction.Cancel();
	}
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchyElementViewModel::CanContain(TSubclassOf<UHierarchyElement> HierarchyElementType)
{
	// Editability check
	{
		FResultWithUserFeedback Results = IsEditableByUser();
		if(Results.bResult == false)
		{
			return Results;
		}
	}

	// Allow children check
	{
		FResultWithUserFeedback Results = CanHaveChildren();
		if(Results.bResult == false)
		{
			return Results;
		}
	}

	// Internal check
	return CanContainInternal(HierarchyElementType);
}

bool FHierarchyElementViewModel::IsExpandedByDefaultInternal() const
{
	// By default, we expand all high-level elements
	return GetHierarchyDepth() <= 1;
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchyElementViewModel::CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedElement, EItemDropZone ItemDropZone)
{
	return true;
}

void FHierarchyElementViewModel::PropagateOnChildSynced()
{
	OnSyncedDelegate.ExecuteIfBound();
}

FHierarchyRootViewModel::~FHierarchyRootViewModel()
{
	
}

void FHierarchyRootViewModel::Initialize()
{
	GetOnSynced().BindSP(this, &FHierarchyRootViewModel::PropagateOnSynced);
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchyRootViewModel::CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType)
{
	// The root generally accepts items and categories as children; sections are not handled as typical children, but they are considered as such for this function.
	FResultWithUserFeedback Results(InHierarchyElementType->IsChildOf<UHierarchyItem>() || InHierarchyElementType->IsChildOf<UHierarchyCategory>() || InHierarchyElementType->IsChildOf<UHierarchySection>());
	return Results;
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchyRootViewModel::CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedElement, EItemDropZone ItemDropZone)
{
	FResultWithUserFeedback Results(true);

	// We don't allow sections to be dropped on the root directly. This disables the Root Drop Zone while still allowing the drop between sections
	if(DraggedElement->GetData()->IsA<UHierarchySection>() && ItemDropZone == EItemDropZone::OntoItem)
	{
		return false;
	}
	
	// If we are not in the All section, we only accept categories to drop directly on the root
	if (TSharedPtr<FHierarchySectionViewModel> HierarchySectionViewModel = HierarchyViewModel->GetActiveHierarchySectionViewModel())
	{
		if(HierarchySectionViewModel->IsAllSection() == false && !DraggedElement->GetData()->IsA<UHierarchyCategory>())
		{
			Results.bResult = false;
			FText BaseMessage = LOCTEXT("RootCanOnlyContainCategories", "While section {0} is active, the root only accepts categories. Switch to the All section first.");
			Results.UserFeedback = FText::FormatOrdered(BaseMessage, HierarchySectionViewModel->GetSectionNameAsText());
		}
	}
	else
	{
		UE_LOGF(LogDataHierarchyEditor, Warning, "HierarchyViewModel (%ls) has a null section view model", *ToString());
	}

	return Results;
}

TSharedPtr<FHierarchySectionViewModel> FHierarchyRootViewModel::AddSection()
{
	FScopedTransaction ScopedTransaction(LOCTEXT("NewSectionAdded", "Added Section"));
	HierarchyViewModel->GetHierarchyRoot()->Modify();
	
	TSet<FName> ExistingSectionNames;
    	
	for(FName& SectionName : GetData<UHierarchyRoot>()->GetSectionNames())
	{
		ExistingSectionNames.Add(SectionName);
	}

	FName NewName = UE::DataHierarchyEditor::GetUniqueName(FName(LOCTEXT("HierarchyEditorDefaultNewSectionName", "Section").ToString()), ExistingSectionNames);

	UHierarchySection* NewSection = NewObject<UHierarchySection>(GetDataMutable(), HierarchyViewModel->GetSectionDataClass());
	GetDataMutable<UHierarchyRoot>()->GetSectionDataMutable().Insert(NewSection, 0);
	SyncViewModelsToData();
	
	TSharedPtr<FHierarchySectionViewModel> SectionViewModel = SectionViewModels[0];
	ensureMsgf(SectionViewModel->GetData() == NewSection, TEXT("The section view model for a new section should always be at index 0 after syncing view models."));
	
	SectionViewModel->SetSectionName(NewName);
	SectionViewModel->GetDataMutable()->Modify();
	
	HierarchyViewModel->SetActiveHierarchySection(SectionViewModel);
	
	OnSectionAddedDelegate.ExecuteIfBound(SectionViewModel);
	
	TInstancedStruct<FHierarchyChangedPayload_ElementsAdded> Payload = TInstancedStruct<FHierarchyChangedPayload_ElementsAdded>::Make();
	Payload.GetMutable<FHierarchyChangedPayload_ElementsAdded>().AddedElementViewModels.Add(SectionViewModel);
	HierarchyViewModel->OnHierarchyChanged(Payload);
	
	OnSectionsChangedDelegate.ExecuteIfBound();
	
	return SectionViewModel;
}

void FHierarchyRootViewModel::DeleteSection(TSharedPtr<FHierarchyElementViewModel> InSectionViewModel)
{
	TSharedPtr<FHierarchySectionViewModel> SectionViewModel = StaticCastSharedPtr<FHierarchySectionViewModel>(InSectionViewModel);
	GetDataMutable<UHierarchyRoot>()->GetSectionDataMutable().Remove(SectionViewModel->GetDataMutable<UHierarchySection>());
	SectionViewModels.Remove(SectionViewModel);

	OnSectionDeletedDelegate.ExecuteIfBound(SectionViewModel);
	
	TInstancedStruct<FHierarchyChangedPayload_ElementsDeleted> Payload = TInstancedStruct<FHierarchyChangedPayload_ElementsDeleted>::Make();
	Payload.GetMutable<FHierarchyChangedPayload_ElementsDeleted>().DeletedElementViewModels.Add(SectionViewModel);
	HierarchyViewModel->OnHierarchyChanged(Payload);
	
	OnSectionsChangedDelegate.ExecuteIfBound();
}

void FHierarchyRootViewModel::SyncViewModelsToDataInternal()
{
	const UHierarchyRoot* RootData = GetData<UHierarchyRoot>();

	TArray<TSharedPtr<FHierarchySectionViewModel>> NewSectionViewModels;
	TArray<TSharedPtr<FHierarchySectionViewModel>> SectionViewModelsToDelete;
	
	for(TSharedPtr<FHierarchySectionViewModel> SectionViewModel : SectionViewModels)
	{
		if(!RootData->GetSectionData().Contains(SectionViewModel->GetData()))
		{
			SectionViewModelsToDelete.Add(SectionViewModel);
		}
	}

	for (TSharedPtr<FHierarchySectionViewModel> SectionViewModel : SectionViewModelsToDelete)
	{
		SectionViewModel->Delete();
	}
	
	for(UHierarchySection* Section : RootData->GetSectionData())
	{
		TSharedPtr<FHierarchySectionViewModel>* SectionViewModelPtr = SectionViewModels.FindByPredicate([Section](TSharedPtr<FHierarchySectionViewModel> SectionViewModel)
		{
			return SectionViewModel->GetData() == Section;
		});

		TSharedPtr<FHierarchySectionViewModel> SectionViewModel = nullptr;

		if(SectionViewModelPtr)
		{
			SectionViewModel = *SectionViewModelPtr;
		}

		if(SectionViewModel == nullptr)
		{
			SectionViewModel = StaticCastSharedPtr<FHierarchySectionViewModel>(HierarchyViewModel->CreateViewModelForElement(Section, StaticCastSharedRef<FHierarchyRootViewModel>(AsShared())));
			SectionViewModel->SyncViewModelsToData();
		}
		
		NewSectionViewModels.Add(SectionViewModel);
	}

	SectionViewModels.Empty();
	SectionViewModels.Append(NewSectionViewModels);

	for(TSharedPtr<FHierarchySectionViewModel> SectionViewModel : SectionViewModels)
	{
		SectionViewModel->OnChildRequestedDeletion().BindSP(this, &FHierarchyRootViewModel::DeleteSection);
	}

	SectionViewModels.Sort([this](const TSharedPtr<FHierarchySectionViewModel>& ItemA, const TSharedPtr<FHierarchySectionViewModel>& ItemB)
		{
			return
			GetDataMutable<UHierarchyRoot>()->GetSectionData().Find(Cast<UHierarchySection>(ItemA->GetDataMutable()))
				<
			GetDataMutable<UHierarchyRoot>()->GetSectionData().Find(Cast<UHierarchySection>(ItemB->GetDataMutable())); 
		});
}

void FHierarchyRootViewModel::PostOnDroppedOn(TSharedPtr<FHierarchyElementViewModel> DroppedElementViewModel)
{
	// If we are in the all section, we clear the section association
	if(HierarchyViewModel->GetActiveHierarchySectionViewModel()->IsAllSection())
	{
		DroppedElementViewModel->GetDataMutable()->DeleteMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
	}
	// If not, we update the section association
	else
	{
		FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = DroppedElementViewModel->GetDataMutable()->FindOrAddMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
		SectionAssociation->Section = HierarchyViewModel->GetActiveHierarchySectionViewModel()->GetSectionData();
	}
}

void FHierarchyRootViewModel::PropagateOnSynced()
{
	OnSyncPropagatedDelegate.ExecuteIfBound();
}

FHierarchySectionViewModel::FHierarchySectionViewModel(UHierarchySection* InItem, TSharedRef<FHierarchyRootViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel)
	: FHierarchyElementViewModel(InItem, InParent, InHierarchyViewModel, InParent->IsForHierarchy())
{
}

const UHierarchySection* FHierarchySectionViewModel::GetSectionData() const
{
	if(IsAllSection())
	{
		return nullptr;
	}

	return GetData<UHierarchySection>();
}

FString FHierarchySectionViewModel::ToString() const
{
	return GetSectionNameAsText().ToString();
}

void FHierarchySectionViewModel::SetSectionName(FName InSectionName)
{
	Cast<UHierarchySection>(Element)->SetSectionName(InSectionName);
}

FName FHierarchySectionViewModel::GetSectionName() const
{
	if(UHierarchySection* Section = Cast<UHierarchySection>(Element))
	{
		return Section->GetSectionName();
	}

	return NAME_None;
}

void FHierarchySectionViewModel::SetSectionNameAsText(const FText& Text)
{
	Cast<UHierarchySection>(Element)->SetSectionNameAsText(Text);
}

FText FHierarchySectionViewModel::GetSectionNameAsText() const
{
	return Cast<UHierarchySection>(Element)->GetSectionNameAsText();
}

FText FHierarchySectionViewModel::GetSectionTooltip() const
{
	return Cast<UHierarchySection>(Element)->GetTooltip();
}

bool FHierarchySectionViewModel::AllowEditingInDetailsPanel() const
{
	return IsForHierarchy() && !IsAllSection();
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchySectionViewModel::IsEditableByUser()
{
	// We generally leave the All section for the hierarchy editable as we still need to allow drag & drop on it
	return FResultWithUserFeedback(IsForHierarchy());
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchySectionViewModel::CanDragInternal()
{
	return IsForHierarchy() && !IsAllSection();
}

bool FHierarchySectionViewModel::CanRenameInternal()
{
	return IsForHierarchy() && !IsAllSection();
}

bool FHierarchySectionViewModel::CanDeleteInternal()
{
	return IsForHierarchy() && !IsAllSection();
}

TOptional<bool> FHierarchySectionViewModel::OnDroppedOnOverride(TSharedPtr<FHierarchyElementViewModel> DroppedElementViewModel, EItemDropZone ItemDropZone)
{
	if(DroppedElementViewModel->GetData()->IsA<UHierarchySection>())
	{
		FScopedTransaction Transaction(LOCTEXT("Transaction_OnSectionMoved", "Moved section"));
		HierarchyViewModel->GetHierarchyRoot()->Modify();
	
		UHierarchySection* DraggedSectionData = DroppedElementViewModel->GetDataMutable<UHierarchySection>();
	
		int32 IndexOfThis = HierarchyViewModel->GetHierarchyRoot()->GetSectionData().Find(GetDataMutable<UHierarchySection>());
		int32 DraggedSectionIndex = HierarchyViewModel->GetHierarchyRoot()->GetSectionData().Find(DraggedSectionData);
	
		TArray<TObjectPtr<UHierarchySection>>& SectionData = HierarchyViewModel->GetHierarchyRoot()->GetSectionDataMutable();
		int32 Count = SectionData.Num();
	
		bool bDropSucceeded = false;
		// above constitutes to the left here
		if(ItemDropZone == EItemDropZone::AboveItem)
		{
			SectionData.RemoveAt(DraggedSectionIndex);
			SectionData.Insert(DraggedSectionData, FMath::Max(IndexOfThis, 0));
	
			bDropSucceeded = true;
		}
		else if(ItemDropZone == EItemDropZone::BelowItem)
		{
			SectionData.RemoveAt(DraggedSectionIndex);
	
			if(IndexOfThis + 1 > SectionData.Num())
			{
				SectionData.Add(DraggedSectionData);
			}
			else
			{
				SectionData.Insert(DraggedSectionData, FMath::Min(IndexOfThis+1, Count));
			}
	
			bDropSucceeded = true;
		}
	
		if(bDropSucceeded)
		{
			HierarchyViewModel->ForceFullRefresh();
			HierarchyViewModel->OnHierarchyChanged(TInstancedStruct<FHierarchyChangedPayload>::Make("Dropped section next to section"));
		}
		
		return true;
	}

	if(UHierarchyElement* DroppedElement = DroppedElementViewModel->GetDataMutable())
	{
		FScopedTransaction Transaction(LOCTEXT("Transaction_OnSectionDrop", "Moved element to section"));
		HierarchyViewModel->GetHierarchyRoot()->Modify();
		
		// If we are in the all section, we clear the section association
		if(IsAllSection())
		{
			DroppedElement->DeleteMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
		}
		// If not, we update the section association
		else
		{
			FDataHierarchyElementMetaData_SectionAssociation* SectionAssociation = DroppedElement->FindOrAddMetaDataOfType<FDataHierarchyElementMetaData_SectionAssociation>();
			SectionAssociation->Section = GetSectionData();
		}
		
		// we only need to reparent if the parent isn't already the root. This stops unnecessary reordering
		if(DroppedElementViewModel->GetParent() != HierarchyViewModel->GetHierarchyRootViewModel())
		{
			HierarchyViewModel->GetHierarchyRootViewModel()->ReparentToThis(DroppedElementViewModel);
		}
		
		HierarchyViewModel->RefreshHierarchyView();
		HierarchyViewModel->OnHierarchyChanged(TInstancedStruct<FHierarchyChangedPayload>::Make("Dropped element on section"));
		
		return true;
	}

	return true;
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchySectionViewModel::CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType)
{
	// While a section doesn't actually 'contain' a category as a child, semantically it gets treated as such in the drag & drop logic
	if(InHierarchyElementType->IsChildOf<UHierarchyCategory>())
	{
		return true;
	}

	FResultWithUserFeedback Results(false);
	Results.UserFeedback = LOCTEXT("SectionsCanOnlyContainCategoriesFeedback", "Sections can only contain categories.");
	return Results;
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchySectionViewModel::CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedElementViewModel, EItemDropZone ItemDropZone)
{
	FResultWithUserFeedback Results(false);
	// we don't allow dropping onto source sections and we don't specify a message as the sections aren't going to light up as valid drop targets
	if(IsForHierarchy() == false)
	{
		return false;
	}
	
	if(UHierarchySection* DraggedSection = Cast<UHierarchySection>(DraggedElementViewModel->GetDataMutable()))
	{
		TSharedPtr<FHierarchySectionViewModel> DraggedSectionViewModel = StaticCastSharedPtr<FHierarchySectionViewModel>(DraggedElementViewModel);
		const bool bSameSection = GetSectionData() == DraggedSectionViewModel->GetSectionData();

		// If we drag a section onto a section, nothing happens
		if(ItemDropZone == EItemDropZone::OntoItem)
		{
			Results.bResult = false;
			return Results;
		}

		// The 'All' section does not accept any section drop actions.
		if(IsAllSection())
		{
			Results.bResult = false;
			return Results;
		}
		
		int32 DraggedSectionIndex = GetHierarchyViewModel()->GetHierarchyRoot()->GetSectionIndex(DraggedSection);
		int32 ThisSectionIndex = GetHierarchyViewModel()->GetHierarchyRoot()->GetSectionIndex(GetDataMutable<UHierarchySection>());

		// We add +1 if below the item, or -1 if above the item, to determine if we'd land on the same index as we currently are.
		// For example, if we have [Section1] [Section2] [Section3], and we drag [Section2] onto the left part of [Section3] or right part of [Section1], nothing should happen
		ThisSectionIndex += ItemDropZone == EItemDropZone::AboveItem ? -1 : 1;
		
		Results.bResult = !bSameSection && DraggedSectionIndex != ThisSectionIndex;

		if(Results.bResult)
		{
			if(ItemDropZone != EItemDropZone::OntoItem)
			{
				FText Message = LOCTEXT("MoveSectionLeftDragMessage", "Move section {0} here");
				Message = FText::FormatOrdered(Message, FText::FromString(DraggedElementViewModel->ToString()));
				Results.UserFeedback = Message;
			}
		}
	}
	// For any element other than sections, we let the CanContain check handle that will be called before this filter out invalid element types
	// If it's a valid type, we assume it's generally allowed to be dropped onto a section
	else
	{
		if(ItemDropZone == EItemDropZone::OntoItem)
		{
			// We only allow adding to a section if the element isn't already part of that section
			FDataHierarchyElementMetaData_SectionAssociation SectionAssociation = DraggedElementViewModel->GetData()->FindMetaDataOfTypeOrDefault<FDataHierarchyElementMetaData_SectionAssociation>();
			
			Results.bResult = SectionAssociation.Section != GetSectionData();
			if(Results.bResult == true)
			{
				FText Message = LOCTEXT("AssociateElementWithSectionDragMessage", "Associate element {0} with section {1}");
				Results.UserFeedback = FText::FormatOrdered(Message, FText::FromString(DraggedElementViewModel->ToString()), GetSectionNameAsText()); 
			}
		}
		
		return Results;
	}

	return Results;
}

void FHierarchySectionViewModel::FinalizeInternal()
{
	if(HierarchyViewModel->GetActiveHierarchySectionViewModel() == AsShared())
	{
		HierarchyViewModel->SetActiveHierarchySection(HierarchyViewModel->GetDefaultHierarchySectionViewModel());
	}
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchyItemViewModel::CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElement)
{
	// By default, items can't contain other children
	return false;
}

FHierarchyElementViewModel::FResultWithUserFeedback FHierarchyItemViewModel::CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedElement, EItemDropZone ItemDropZone)
{
	// items can be generally be dropped above/below other items
	return (GetData()->IsA<UHierarchyItem>() && ItemDropZone != EItemDropZone::OntoItem);
}

const FTableRowStyle* FHierarchyItemViewModel::GetRowStyle() const
{
	return &FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");
}

const FTableRowStyle* FHierarchyCategoryViewModel::GetRowStyle() const
{
	return &FDataHierarchyEditorStyle::Get().GetWidgetStyle<FTableRowStyle>("HierarchyEditor.Row.Category");
}

void UDataHierarchyViewModelBase::AddCategory(TSharedPtr<FHierarchyElementViewModel> CategoryParent)
{
	// If no category parent was specified, we add it to the root
	if(CategoryParent == nullptr)
	{
		CategoryParent = GetHierarchyRootViewModel();	
	}
	
	int32 HierarchyDepth = CategoryParent->GetHierarchyDepth();
	if(HierarchyDepth > 15)
	{
		FNotificationInfo Info(LOCTEXT("TooManyNestedCategoriesToastText", "We currently only allow a hierarchy depth of 15."));
		Info.ExpireDuration = 4.f;
		FSlateNotificationManager::Get().AddNotification(Info);
		return;
	}

	FText TransactionText = FText::FormatOrdered(LOCTEXT("Transaction_AddedItem", "Added new {0} to hierarchy"), FText::FromString(GetCategoryDataClass()->GetName()));
	FScopedTransaction Transaction(TransactionText);
	GetHierarchyRoot()->Modify();

	UClass* CategoryClass = GetCategoryDataClass();
	
	TSharedPtr<FHierarchyCategoryViewModel> CategoryViewModel = StaticCastSharedPtr<FHierarchyCategoryViewModel>(CategoryParent->AddChild(CategoryClass, INDEX_NONE, UHierarchyCategory::ConstructIdentity()));
	if(ensureMsgf(CategoryViewModel.IsValid(), TEXT("Could not find view model for new category of type '%s'. Please ensure your 'CreateViewModelForData' function creates a view model."), *CategoryClass->GetName()))
	{		
		TArray<UHierarchyCategory*> SiblingCategories;

		CategoryViewModel->GetParent().Pin()->GetData()->GetChildrenOfType<UHierarchyCategory>(SiblingCategories);
		
		TSet<FName> CategoryNames;
		for(const auto& SiblingCategory : SiblingCategories)
		{
			CategoryNames.Add(SiblingCategory->GetCategoryName());
		}

		CategoryViewModel->SetCategoryName(UE::DataHierarchyEditor::GetUniqueName(FName("New Category"), CategoryNames));
	}
}

void UDataHierarchyViewModelBase::GenerateDynamicContextMenu(UToolMenu* ToolMenu)
{
	UHierarchyMenuContext* HierarchyMenuContext = ToolMenu->FindContext<UHierarchyMenuContext>();

	if(HierarchyMenuContext == nullptr || HierarchyMenuContext->HierarchyViewModel.IsValid() == false)
	{
		return;
	}

	UDataHierarchyViewModelBase* HierarchyViewModel = HierarchyMenuContext->HierarchyViewModel.Get();
	HierarchyViewModel->GenerateDynamicContextMenuInternal(ToolMenu);
	
	if(HierarchyMenuContext->MenuHierarchyElements.Num() == 1)
	{
		HierarchyMenuContext->MenuHierarchyElements[0]->AppendDynamicContextMenuForSingleElement(ToolMenu);
	}
}

void UDataHierarchyViewModelBase::GenerateDynamicContextMenuInternal(UToolMenu* DynamicToolMenu)
{
	UHierarchyMenuContext* HierarchyMenuContext = DynamicToolMenu->FindContext<UHierarchyMenuContext>();

	if(HierarchyMenuContext == nullptr || HierarchyMenuContext->HierarchyViewModel.IsValid() == false)
	{
		return;
	}

	// Disclaimer: We aren't simply using the commands with the command list, because the regular commandlist logic does not allow us to pass in tool menu contexts.
	// CommandLists work with shortcuts, meaning there might not be a menu to work with.
	// This means for something like a 'Delete' action, we have to figure out 'what' to delete without context, which is difficult since we have 2 tree views & selected sections.
	// When we DO have a menu, we want to use the better ToolMenu logic that tells us what elements we are operating on.
	
	// Find
	{
		FToolUIAction Action;
		Action.ExecuteAction = FToolMenuExecuteAction::CreateUObject(this, &UDataHierarchyViewModelBase::ToolMenuNavigateTo);
		Action.CanExecuteAction = FToolMenuCanExecuteAction::CreateUObject(this, &UDataHierarchyViewModelBase::ToolMenuCanNavigateTo);
		Action.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateUObject(this, &UDataHierarchyViewModelBase::ToolMenuCanNavigateTo);

		TSharedPtr<FUICommandInfo> Command = FDataHierarchyEditorCommands::Get().FindInHierarchy;
		FToolMenuEntry MenuEntry = FToolMenuEntry::InitMenuEntry(Command->GetCommandName(), Command->GetLabel(), Command->GetDescription(), Command->GetIcon(), Action);
		MenuEntry.InputBindingLabel = Command->GetInputText();
		DynamicToolMenu->AddMenuEntry("Dynamic", MenuEntry);
	}
	
	// Rename
	{
		FToolUIAction Action;
		Action.ExecuteAction = FToolMenuExecuteAction::CreateUObject(this, &UDataHierarchyViewModelBase::ToolMenuRequestRename);
		Action.CanExecuteAction = FToolMenuCanExecuteAction::CreateUObject(this, &UDataHierarchyViewModelBase::ToolMenuCanRequestRename);
		Action.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateUObject(this, &UDataHierarchyViewModelBase::ToolMenuCanRequestRename);

		TSharedPtr<FUICommandInfo> Command = FGenericCommands::Get().Rename;
		FToolMenuEntry MenuEntry = FToolMenuEntry::InitMenuEntry(Command->GetCommandName(), Command->GetLabel(), Command->GetDescription(), Command->GetIcon(), Action);
		MenuEntry.InputBindingLabel = Command->GetInputText();
		DynamicToolMenu->AddMenuEntry("Dynamic", MenuEntry);
	}

	// Delete
	{
		FToolUIAction Action;
		Action.ExecuteAction = FToolMenuExecuteAction::CreateUObject(this, &UDataHierarchyViewModelBase::ToolMenuDelete);
		Action.CanExecuteAction = FToolMenuCanExecuteAction::CreateUObject(this, &UDataHierarchyViewModelBase::ToolMenuCanDelete);
		Action.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateUObject(this, &UDataHierarchyViewModelBase::ToolMenuCanDelete);
		
		TSharedPtr<FUICommandInfo> Command = FGenericCommands::Get().Delete;
		FToolMenuEntry MenuEntry = FToolMenuEntry::InitMenuEntry(Command->GetCommandName(), Command->GetLabel(), Command->GetDescription(), Command->GetIcon(), Action);
		MenuEntry.InputBindingLabel = Command->GetInputText();
		DynamicToolMenu->AddMenuEntry("Dynamic", MenuEntry);
	}
}

void UDataHierarchyViewModelBase::AddElementUnderRoot(TSubclassOf<UHierarchyElement> NewChildClass, FHierarchyElementIdentity ChildIdentity)
{
	FScopedTransaction Transaction(LOCTEXT("Transaction_AddItem", "Add hierarchy item"));
	HierarchyRoot->Modify();
	GetHierarchyRootViewModel()->AddChild(NewChildClass, INDEX_NONE, ChildIdentity);
}

void UDataHierarchyViewModelBase::DeleteElementWithIdentity(FHierarchyElementIdentity Identity)
{
	if(Identity.IsValid() == false)
	{
		return;
	}
	
	FScopedTransaction Transaction(LOCTEXT("Transaction_DeleteItem", "Deleted hierarchy item"));
	HierarchyRoot->Modify();

	TSharedPtr<FHierarchyElementViewModel> ViewModelToDelete;
	
	if(TSharedPtr<FHierarchyElementViewModel> ViewModel = HierarchyRootViewModel->FindViewModelForChild(Identity, true))
	{
		if(ViewModel->CanDelete())
		{
			ViewModelToDelete = ViewModel;
		}
	}
	
	// If we haven't found an element in the hierarchy, check the sections
	if (!ViewModelToDelete.IsValid())
	{
		for(TSharedPtr<FHierarchySectionViewModel> SectionViewModel : HierarchyRootViewModel->GetSectionViewModels())
		{
			if(SectionViewModel->GetData()->GetPersistentIdentity() == Identity && SectionViewModel->CanDelete())
			{
				ViewModelToDelete = SectionViewModel;
			}
		}
	}

	if (ViewModelToDelete.IsValid())
	{
		ViewModelToDelete->Delete();
		
		TInstancedStruct<FHierarchyChangedPayload_ElementsDeleted> Payload = TInstancedStruct<FHierarchyChangedPayload_ElementsDeleted>::Make();
		Payload.GetMutable<FHierarchyChangedPayload_ElementsDeleted>().DeletedElementViewModels.Add(ViewModelToDelete);
		
		HierarchyRootViewModel->SyncViewModelsToData();
		
		OnHierarchyChanged(Payload);
	}
	else
	{
		Transaction.Cancel();
	}
}

void UDataHierarchyViewModelBase::DeleteElements(TArray<TSharedPtr<FHierarchyElementViewModel>> ViewModels)
{
	FScopedTransaction Transaction(LOCTEXT("Transaction_DeleteHierarchyElements", "Deleted hierarchy elements"));
	HierarchyRoot->Modify();
	
	bool bAnyItemsDeleted = false;
	
	TArray<TSharedPtr<FHierarchyElementViewModel>> ViewModelsToDelete;
	for(TSharedPtr<FHierarchyElementViewModel> ViewModel : ViewModels)
	{
		if(ViewModel->CanDelete())
		{
			ViewModelsToDelete.Add(ViewModel);
			
		}
	}
	
	for (TSharedPtr<FHierarchyElementViewModel> ViewModelToDelete : ViewModelsToDelete)
	{
		ViewModelToDelete->Delete();
		bAnyItemsDeleted = true;
	}

	if(bAnyItemsDeleted)
	{
		HierarchyRootViewModel->SyncViewModelsToData();
		
		TInstancedStruct<FHierarchyChangedPayload_ElementsDeleted> Payload = TInstancedStruct<FHierarchyChangedPayload_ElementsDeleted>::Make();
		Payload.GetMutablePtr<FHierarchyChangedPayload_ElementsDeleted>()->DeletedElementViewModels = ViewModelsToDelete;
		OnHierarchyChanged(Payload);
	}
	else
	{
		Transaction.Cancel();
	}
}

void UDataHierarchyViewModelBase::NavigateToElementInHierarchy(const FHierarchyElementIdentity& HierarchyIdentity) const
{
	OnNavigateToElementIdentityInHierarchyRequestedDelegate.ExecuteIfBound(HierarchyIdentity);
}

void UDataHierarchyViewModelBase::NavigateToElementInHierarchy(const TSharedRef<FHierarchyElementViewModel> HierarchyElement) const
{
	OnNavigateToElementInHierarchyRequestedDelegate.ExecuteIfBound(HierarchyElement);
}

TArray<FDraggedElementEntry> FDataHierarchyDragDropContext::GetTopLevelEntries() const
{
	TArray<FDraggedElementEntry> TopLevel;
	for(const FDraggedElementEntry& Entry : Entries)
	{
		if(!Entry.bIsContainedInAnotherDraggedElement)
		{
			TopLevel.Add(Entry);
		}
	}
	return TopLevel;
}

int32 FDataHierarchyDragDropContext::GetTopLevelCount() const
{
	int32 Count = 0;
	for(const FDraggedElementEntry& Entry : Entries)
	{
		if(!Entry.bIsContainedInAnotherDraggedElement)
		{
			Count++;
		}
	}
	return Count;
}

FDataHierarchyDragDropContext::~FDataHierarchyDragDropContext() = default;
FDataHierarchyDragDropContext::FDataHierarchyDragDropContext(const TArray<FDraggedElementEntry>& InEntries) : Entries(InEntries) {}
FDataHierarchyDragDropContext::FDataHierarchyDragDropContext(TArray<FDraggedElementEntry>&& InEntries) : Entries(MoveTemp(InEntries)) {}

const TArray<FDraggedElementEntry>& FDataHierarchyDragDropContext::GetEntries() const { return Entries; }
TWeakPtr<FHierarchyElementViewModel> FDataHierarchyDragDropContext::GetPrimaryElement() const { return Entries.Num() > 0 ? Entries[0].Element : nullptr; }
int32 FDataHierarchyDragDropContext::Num() const { return Entries.Num(); }
bool FDataHierarchyDragDropContext::IsEmpty() const { return Entries.IsEmpty(); }

FHierarchyDragDropOp::FHierarchyDragDropOp(const FDataHierarchyDragDropContext& InDragContext) : DragContext(InDragContext)
{
	TWeakPtr<FHierarchyElementViewModel> Primary = DragContext.GetPrimaryElement();
	if(Primary.IsValid())
	{
		int32 TopLevelCount = DragContext.GetTopLevelCount();
		FText PrimaryLabel = Primary.Pin()->ToStringAsText();

		if(TopLevelCount > 1)
		{
			SetLabel(FText::Format(NSLOCTEXT("DataHierarchyEditor", "MultiDragLabel", "{0} (+{1})"), PrimaryLabel, TopLevelCount - 1));
		}
		else
		{
			SetLabel(PrimaryLabel);
		}
	}
}

TSharedPtr<SWidget> FHierarchyDragDropOp::GetDefaultDecorator() const
{
	TSharedRef<SWidget> CustomDecorator = CreateCustomDecorator();

	SVerticalBox::FSlot* CustomSlot;
	TSharedPtr<SWidget> Decorator = SNew(SToolTip)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Expose(CustomSlot).
		AutoHeight()
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.f)
		[
			SNew(STextBlock)
			.Text(this, &FHierarchyDragDropOp::GetLabel)
			.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText.Important"))
			.Visibility_Lambda([this, CustomDecorator]()
			{
				return GetLabel().IsEmpty() || CustomDecorator != SNullWidget::NullWidget ? EVisibility::Collapsed : EVisibility::Visible;
			})
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.f)
		[
			SNew(STextBlock)
			.Text(this, &FHierarchyDragDropOp::GetDescription)
			.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText"))
			.Visibility_Lambda([this]()
			{
				return GetDescription().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
			})
		]
	];

	if(CustomDecorator != SNullWidget::NullWidget)
	{
		CustomSlot->AttachWidget(CustomDecorator);
	}

	return Decorator;
}

TSharedRef<SWidget> FSectionDragDropOp::CreateCustomDecorator() const
{
	return SNew(SCheckBox)
		.Visibility(EVisibility::HitTestInvisible)
		.Style(FAppStyle::Get(), "DetailsView.SectionButton")
		.IsChecked(ECheckBoxState::Unchecked)
		[
			SNew(SInlineEditableTextBlock)
			.Text(GetDraggedSection().Pin()->GetSectionNameAsText())
		];
}

#undef LOCTEXT_NAMESPACE
