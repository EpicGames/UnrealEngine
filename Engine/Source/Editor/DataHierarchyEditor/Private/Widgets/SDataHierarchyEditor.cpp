// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDataHierarchyEditor.h"

#include "DataHierarchyEditorCommands.h"
#include "DataHierarchyEditorMisc.h"
#include "UObject/Package.h"
#include "Framework/Commands/GenericCommands.h"
#include "Styling/StyleColors.h"
#include "Styling/AppStyle.h"
#include "DataHierarchyViewModelBase.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "ScopedTransaction.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "ToolMenus.h"
#include "DataHierarchyEditorStyle.h"
#include "SPositiveActionButton.h"
#include "Algo/AnyOf.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SToolTip.h"

#define LOCTEXT_NAMESPACE "DataHierarchyEditor"

TSharedRef<SWidget> SummonContextMenu(TArray<TSharedPtr<FHierarchyElementViewModel>> MenuHierarchyElements)
{
	UHierarchyMenuContext* MenuContextObject = NewObject<UHierarchyMenuContext>();
	MenuContextObject->MenuHierarchyElements = MenuHierarchyElements;
	TWeakObjectPtr<UDataHierarchyViewModelBase> ViewModel = MenuHierarchyElements[0]->GetHierarchyViewModel();
	MenuContextObject->HierarchyViewModel = ViewModel;

	FToolMenuContext MenuContext(MenuContextObject);
	MenuContext.AppendCommandList(ViewModel->GetCommands());
	
	TSharedRef<SWidget> MenuWidget = UToolMenus::Get()->GenerateWidget(ViewModel->GetContextMenuName(), MenuContext);
	return MenuWidget;
}

void SHierarchyElement::Construct(const FArguments& InArgs, TSharedPtr<struct FHierarchyElementViewModel> InViewModel)
{
	ElementViewModelWeak = InViewModel;
	
	InViewModel->GetOnRequestRename().BindSP(this, &SHierarchyElement::EnterEditingMode);
	
	SetToolTipText(TAttribute<FText>::CreateSP(this, &SHierarchyElement::GetTooltipText));
	
	ChildSlot
	[
		SAssignNew(InlineEditableTextBlock, SInlineEditableTextBlock)
		.Style(InArgs._Style)
		.Text(this, &SHierarchyElement::GetElementText)
		.OnTextCommitted(this, &SHierarchyElement::OnRenameElement)
		.OnVerifyTextChanged(this, &SHierarchyElement::OnVerifyRename)
		.IsSelected(InArgs._IsSelected)
	];
}

void SHierarchyElement::EnterEditingMode() const
{
	if(ElementViewModelWeak.Pin()->CanRename())
	{
		InlineEditableTextBlock->EnterEditingMode();
	}
}

bool SHierarchyElement::OnVerifyRename(const FText& NewName, FText& OutTooltip) const
{
	TArray<TSharedPtr<FHierarchyItemViewModel>> SiblingItemViewModels;
	ElementViewModelWeak.Pin()->GetParent().Pin()->GetChildrenViewModelsForType<UHierarchyElement, FHierarchyItemViewModel>(SiblingItemViewModels);

	if(GetElementText().ToString() != NewName.ToString())
	{
		TSet<FString> ItemNames;
		for(const auto& SiblingItemViewModel : SiblingItemViewModels)
		{
			ItemNames.Add(Cast<UHierarchyElement>(SiblingItemViewModel->GetDataMutable())->ToString());
		}

		if(ItemNames.Contains(NewName.ToString()))
		{
			OutTooltip = LOCTEXT("HierarchyItemCantRename_DuplicateOnLayer", "Another item of the same name already exists!");
			return false;
		}
	}

	return true;
}

FText SHierarchyElement::GetElementText() const
{
	return FText::FromString(ElementViewModelWeak.Pin()->ToString());
}

void SHierarchyElement::OnRenameElement(const FText& NewText, ETextCommit::Type) const
{
	if(ElementViewModelWeak.IsValid() && ElementViewModelWeak.Pin()->ToStringAsText().EqualTo(NewText) == false)
	{
		FScopedTransaction Transaction(LOCTEXT("Transaction_Rename_Item", "Renamed hierarchy item"));
		ElementViewModelWeak.Pin()->GetHierarchyViewModel()->GetHierarchyRoot()->Modify();
		
		ElementViewModelWeak.Pin()->Rename(FName(NewText.ToString()));
	}
}

FText SHierarchyElement::GetTooltipText() const
{
	if (ElementViewModelWeak.IsValid())
	{
		if (const UHierarchyCategory* Category = Cast<UHierarchyCategory>(ElementViewModelWeak.Pin()->GetData()))
		{
			return Category->GetTooltip();
		}
	}
	
	return FText::GetEmpty();
}

void SHierarchySection::Construct(const FArguments& InArgs, TSharedPtr<FHierarchySectionViewModel> InSection)
{
	SectionViewModelWeak = InSection;
	HierarchyViewModel = InSection->GetHierarchyViewModel();

	IsSectionActive = InArgs._IsSectionActive;
	OnSectionActivatedDelegate = InArgs._OnSectionActivated;
	
	InSection->GetOnRequestRename().BindSP(this, &SHierarchySection::TryEnterEditingMode);

	SetToolTipText(TAttribute<FText>::CreateSP(this, &SHierarchySection::GetTooltipText));

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Fill)
		[
			SAssignNew(CheckBox, SCheckBox)
			.Visibility(EVisibility::HitTestInvisible)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.OnCheckStateChanged(this, &SHierarchySection::OnSectionCheckChanged)
			.IsChecked(this, &SHierarchySection::GetSectionCheckState)
			.Padding(FMargin(8.f, 4.f))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f)
				[
					SNew(SBox)
					.WidthOverride(20.f)
					.HeightOverride(20.f)
					.Visibility(this, &SHierarchySection::GetImageVisibility)
					[
						SNew(SImage)
						.Image(this, &SHierarchySection::GetImageBrush)
					]
				]
				+ SHorizontalBox::Slot()
				[
					SAssignNew(InlineEditableTextBlock, SInlineEditableTextBlock)
					.Visibility(EVisibility::HitTestInvisible)
					.Text(this, &SHierarchySection::GetText)
					.OnTextCommitted(this, &SHierarchySection::OnRenameSection)
					.OnVerifyTextChanged(this, &SHierarchySection::OnVerifySectionRename)
					.IsSelected(this, &SHierarchySection::IsSectionSelected)
					.IsReadOnly(this, &SHierarchySection::IsSectionReadOnly)
				]
			]
		]
	];		
}

SHierarchySection::~SHierarchySection()
{
	SectionViewModelWeak.Reset();
}

void SHierarchySection::TryEnterEditingMode() const
{
	if(SectionViewModelWeak.IsValid() && SectionViewModelWeak.Pin()->CanRename())
	{
		InlineEditableTextBlock->EnterEditingMode();
	}
}

TSharedPtr<FHierarchySectionViewModel> SHierarchySection::GetSectionViewModel()
{
	return SectionViewModelWeak.Pin();
}

int32 SHierarchySection::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId =  SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	LayerId = PaintDropIndicator(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	return LayerId;
}

int32 SHierarchySection::PaintDropIndicator(const FPaintArgs& Args, const FGeometry& Geometry, FSlateRect SlateRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& WidgetStyle, bool bParentEnabled) const
{
	if(CurrentItemDropZone.IsSet())
	{
		return OnPaintDropIndicator(CurrentItemDropZone.GetValue(), Args, Geometry, SlateRect, OutDrawElements, LayerId, WidgetStyle, bParentEnabled);
	}

	return LayerId;
}

int32 SHierarchySection::OnPaintDropIndicator(EItemDropZone InItemDropZone, const FPaintArgs& Args, const FGeometry& Geometry, FSlateRect SlateRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& WidgetStyle, bool bParentEnabled) const
{
	const FSlateBrush* DropIndicatorBrush = GetDropIndicatorBrush(InItemDropZone);

	const FVector2f LocalSize(Geometry.GetLocalSize());
	const FVector2f Pivot(LocalSize * 0.5f);
	const FVector2f RotatedLocalSize(LocalSize.Y, LocalSize.X);
	FSlateLayoutTransform RotatedTransform(Pivot - RotatedLocalSize * 0.5f);	// Make the box centered to the alloted geometry, so that it can be rotated around the center.
	
	FSlateDrawElement::MakeRotatedBox(
		OutDrawElements,
		LayerId++,
		Geometry.ToPaintGeometry(RotatedLocalSize, RotatedTransform),
		DropIndicatorBrush,
		ESlateDrawEffect::None,
		-UE_HALF_PI,	// 90 deg CCW
		RotatedLocalSize * 0.5f,	// Relative center to the flipped
		FSlateDrawElement::RelativeToElement,
		DropIndicatorBrush->GetTint(WidgetStyle) * WidgetStyle.GetColorAndOpacityTint()
	);

	return LayerId;
}

bool SHierarchySection::OnCanAcceptDrop(TSharedPtr<FDragDropOperation> DragDropOperation, EItemDropZone InItemDropZone) const
{
	if(DragDropOperation->IsOfType<FHierarchyDragDropOp>() && SectionViewModelWeak.IsValid())
	{
		TSharedPtr<FHierarchyDragDropOp> HierarchyDragDropOp = StaticCastSharedPtr<FHierarchyDragDropOp>(DragDropOperation);
		TArray<FDraggedElementEntry> TopLevelEntries = HierarchyDragDropOp->GetDragContext().GetTopLevelEntries();

		// Accept if at least one entry can be dropped. Partial multi-select is still allowed.
		for(const FDraggedElementEntry& Entry : TopLevelEntries)
		{
			TSharedPtr<FHierarchyElementViewModel> DraggedItem = Entry.Element.Pin();
			if (DraggedItem.IsValid() && SectionViewModelWeak.Pin()->CanDropOn(DraggedItem, InItemDropZone).bResult)
			{
				return true;
			}
		}
	}

	return false;
}

FReply SHierarchySection::OnDroppedOn(const FGeometry&, const FDragDropEvent& DragDropEvent, EItemDropZone DropZone) const
{
	bDraggedOn = false;
	CurrentItemDropZone.Reset();

	if(TSharedPtr<FHierarchyDragDropOp> HierarchyDragDropOp = DragDropEvent.GetOperationAs<FHierarchyDragDropOp>())
	{
		TArray<FDraggedElementEntry> TopLevelEntries = HierarchyDragDropOp->GetDragContext().GetTopLevelEntries();

		FScopedTransaction Transaction(LOCTEXT("Transaction_MultiDropOnSection", "Dropped items on section"));

		for(const FDraggedElementEntry& Entry : TopLevelEntries)
		{
			TSharedPtr<FHierarchyElementViewModel> DraggedItem = Entry.Element.Pin();
			if (DraggedItem.IsValid() && SectionViewModelWeak.Pin()->CanDropOn(DraggedItem, DropZone).bResult)
			{
				SectionViewModelWeak.Pin()->OnDroppedOn(DraggedItem, DropZone);
			}
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SHierarchySection::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// we handle the event here so we can react on mouse button up
	if(MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
	{
		return FReply::Handled();
	}
	else if(MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{			
		OnSectionActivatedDelegate.ExecuteIfBound(SectionViewModelWeak.Pin());
		return FReply::Handled().DetectDrag(AsShared(), EKeys::LeftMouseButton).SetUserFocus(AsShared());	
	}

	return FReply::Unhandled();
}

FReply SHierarchySection::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(SectionViewModelWeak.IsValid() && SectionViewModelWeak.Pin()->IsForHierarchy() && SectionViewModelWeak.Pin()->GetData() != nullptr)
	{
		if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{			
			// Show menu to choose getter vs setter
			FSlateApplication::Get().PushMenu(
				AsShared(),
				FWidgetPath(),
				SummonContextMenu({GetSectionViewModel()}),
				FSlateApplication::Get().GetCursorPos(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);
				
			OnSectionActivatedDelegate.ExecuteIfBound(SectionViewModelWeak.Pin());			
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SHierarchySection::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if(CurrentItemDropZone.IsSet())
	{
		if(OnCanAcceptDrop(DragDropEvent.GetOperation(), CurrentItemDropZone.GetValue()))
		{
			return OnDroppedOn(MyGeometry, DragDropEvent, CurrentItemDropZone.GetValue());
		}
	}

	return FReply::Unhandled();
}

void SHierarchySection::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	bDraggedOn = true;
	if(DragDropEvent.GetOperationAs<FHierarchyDragDropOp>() && DragDropEvent.GetOperationAs<FSectionDragDropOp>() == nullptr)
	{
		RegisterActiveTimer(1.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SHierarchySection::ActivateSectionIfDragging));
	}
}

FReply SHierarchySection::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && SectionViewModelWeak.Pin()->CanDrag().bResult)
	{
		TSharedRef<FSectionDragDropOp> SectionDragDropOp = MakeShared<FSectionDragDropOp>(SectionViewModelWeak.Pin());
		SectionDragDropOp->Construct();
		return FReply::Handled().BeginDragDrop(SectionDragDropOp);
	}
	
	return FReply::Unhandled();
}

void SHierarchySection::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	bDraggedOn = false;
	CurrentItemDropZone.Reset();
	
	if(TSharedPtr<FHierarchyDragDropOp> HierarchyDragDropOp = DragDropEvent.GetOperationAs<FHierarchyDragDropOp>())
	{
		HierarchyDragDropOp->SetDescription(FText::GetEmpty());
	}
}

FReply SHierarchySection::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	const FVector2f LocalPointerPos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
	const EItemDropZone ItemHoverZone = ZoneFromPointerPosition(LocalPointerPos, MyGeometry.GetLocalSize());

	if(TSharedPtr<FHierarchyDragDropOp> HierarchyDragDropOp = DragDropEvent.GetOperationAs<FHierarchyDragDropOp>())
	{
		TArray<FDraggedElementEntry> TopLevelEntries = HierarchyDragDropOp->GetDragContext().GetTopLevelEntries();
		TopLevelEntries.RemoveAll([](FDraggedElementEntry& Entry)
		{
			return !Entry.Element.IsValid();
		});

		const TSharedPtr<FHierarchySectionViewModel> SectionViewModel = SectionViewModelWeak.Pin();

		// Single-drop.
		if (TopLevelEntries.Num() == 1)
		{
			const TSharedPtr<FHierarchyElementViewModel> DraggedItem = TopLevelEntries[0].Element.Pin();
			const FHierarchyElementViewModel::FResultWithUserFeedback Result = SectionViewModel->CanDropOn(DraggedItem, ItemHoverZone);
			if (Result.bResult)
			{
				CurrentItemDropZone = ItemHoverZone;
			}
			else
			{
				CurrentItemDropZone.Reset();
			}
			HierarchyDragDropOp->SetDescription(Result.UserFeedback.Get(FText::GetEmpty()));
			return FReply::Handled();
		}

		// Multi-drop.
		if (TopLevelEntries.Num() > 1)
		{
			int32 ValidCount = 0;
			FText RejectionMessage;
			for (const FDraggedElementEntry& Entry : TopLevelEntries)
			{
				const TSharedPtr<FHierarchyElementViewModel> DraggedItem = Entry.Element.Pin();
				FHierarchyElementViewModel::FResultWithUserFeedback Result = SectionViewModel->CanDropOn(DraggedItem, ItemHoverZone);
				if (Result.bResult)
				{
					++ValidCount;
				}
				else if (RejectionMessage.IsEmpty())
				{
					RejectionMessage = Result.UserFeedback.Get(FText::GetEmpty());
				}
			}

			const int32 TotalCount = TopLevelEntries.Num();
			if (ValidCount == 0)
			{
				CurrentItemDropZone.Reset();
				HierarchyDragDropOp->SetDescription(RejectionMessage);
			}
			else
			{
				CurrentItemDropZone = ItemHoverZone;
				if (ValidCount == TotalCount)
				{
					HierarchyDragDropOp->SetDescription(FText::Format(LOCTEXT("MultiDropOnSection", "Move {0} items to section {1}"), TotalCount, SectionViewModel->GetSectionNameAsText()));
				}
				else
				{
					HierarchyDragDropOp->SetDescription(FText::Format(LOCTEXT("MultiDropOnSectionPartial", "Move {0} of {1} items to section {2}"), ValidCount, TotalCount, SectionViewModel->GetSectionNameAsText()));
				}
			}
			return FReply::Handled();
		}

		// No valid entries to evaluate
		CurrentItemDropZone.Reset();
		HierarchyDragDropOp->SetDescription(FText::GetEmpty());
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SHierarchySection::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if(SectionViewModelWeak.IsValid() && InKeyEvent.GetKey() == EKeys::Delete && SectionViewModelWeak.Pin()->CanDelete())
	{
		SectionViewModelWeak.Pin()->Delete();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SHierarchySection::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);
}

UHierarchySection* SHierarchySection::TryGetSectionData() const
{
	return Cast<UHierarchySection>(SectionViewModelWeak.Pin()->GetDataMutable());
}

const FSlateBrush* SHierarchySection::GetImageBrush() const
{
	return (SectionViewModelWeak.IsValid() && SectionViewModelWeak.Pin()->GetSectionImageBrush() != FAppStyle::GetNoBrush()) ? SectionViewModelWeak.Pin()->GetSectionImageBrush() : FAppStyle::GetNoBrush();
}

EVisibility SHierarchySection::GetImageVisibility() const
{
	return (SectionViewModelWeak.IsValid() && SectionViewModelWeak.Pin()->GetSectionImageBrush() != FAppStyle::GetNoBrush()) ? EVisibility::Visible : EVisibility::Collapsed;  
}

FText SHierarchySection::GetText() const
{
	return SectionViewModelWeak.IsValid() ? SectionViewModelWeak.Pin()->GetSectionNameAsText() : FText::GetEmpty();
}

FText SHierarchySection::GetTooltipText() const
{
	return SectionViewModelWeak.IsValid() ? SectionViewModelWeak.Pin()->GetSectionTooltip() : FText::GetEmpty();
}

void SHierarchySection::OnRenameSection(const FText& Text, ETextCommit::Type CommitType) const
{
	if(SectionViewModelWeak.IsValid() && SectionViewModelWeak.Pin()->GetSectionNameAsText().EqualTo(Text) == false)
	{
		FScopedTransaction Transaction(LOCTEXT("Transaction_Rename_Section", "Renamed hierarchy section"));
		HierarchyViewModel->GetHierarchyRoot()->Modify();
		
		SectionViewModelWeak.Pin()->Rename(FName(Text.ToString()));
	}
}

bool SHierarchySection::OnVerifySectionRename(const FText& NewName, FText& OutTooltip) const
{
	// this function shouldn't be used in case the section isn't valid but we'll make sure regardless
	if(!SectionViewModelWeak.IsValid())
	{
		return false;
	}

	if(SectionViewModelWeak.Pin()->GetSectionName().ToString() != NewName.ToString())
	{
		TArray<FString> SectionNames;

		SectionNames.Add("All");
		for(auto& Section : HierarchyViewModel->GetHierarchyRootViewModel()->GetSectionViewModels())
		{
			SectionNames.Add(Section->GetSectionName().ToString());
		}

		if(SectionNames.Contains(NewName.ToString()))
		{
			OutTooltip = LOCTEXT("HierarchySectionCantRename_Duplicate", "A section with that name already exists!");
			return false;
		}
	}

	return true;
}

bool SHierarchySection::IsSectionSelected() const
{
	return GetSectionCheckState() == ECheckBoxState::Checked ? true : false;
}

bool SHierarchySection::IsSectionReadOnly() const
{
	return SectionViewModelWeak.IsValid() ? !SectionViewModelWeak.Pin()->CanRename() : true;
}

ECheckBoxState SHierarchySection::GetSectionCheckState() const
{
	return IsSectionActive.Get();
}

void SHierarchySection::OnSectionCheckChanged(ECheckBoxState NewState)
{
	OnSectionActivatedDelegate.ExecuteIfBound(SectionViewModelWeak.Pin());
}

EActiveTimerReturnType SHierarchySection::ActivateSectionIfDragging(double CurrentTime, float DeltaTime) const
{
	if(bDraggedOn && FSlateApplication::Get().GetDragDroppingContent().IsValid() && FSlateApplication::Get().GetDragDroppingContent()->IsOfType<FHierarchyDragDropOp>())
	{
		if(IsSectionSelected() == false)
		{
			OnSectionActivatedDelegate.ExecuteIfBound(SectionViewModelWeak.Pin());
		}
	}

	return EActiveTimerReturnType::Stop;
}

const FSlateBrush* SHierarchySection::GetDropIndicatorBrush(EItemDropZone InItemDropZone) const
{
	switch (InItemDropZone)
	{
		case EItemDropZone::AboveItem: return FDataHierarchyEditorStyle::Get().GetBrush("HierarchyEditor.Drop.Section.Left");
		default:
		case EItemDropZone::OntoItem: return FDataHierarchyEditorStyle::Get().GetBrush("HierarchyEditor.Drop.Section.Onto");
		case EItemDropZone::BelowItem: return FDataHierarchyEditorStyle::Get().GetBrush("HierarchyEditor.Drop.Section.Right");
	};
}

EItemDropZone SHierarchySection::ZoneFromPointerPosition(UE::Slate::FDeprecateVector2DParameter LocalPointerPos, UE::Slate::FDeprecateVector2DParameter LocalSize)
{
	const float PointerPos = LocalPointerPos.X;
	const float Size = LocalSize.X;

	const float ZoneBoundarySu = FMath::Clamp(Size * 0.25f, 3.0f, 10.0f);
	if (PointerPos < ZoneBoundarySu)
	{
		return EItemDropZone::AboveItem;
	}
	else if (PointerPos > Size - ZoneBoundarySu)
	{
		return EItemDropZone::BelowItem;
	}
	else
	{
		return EItemDropZone::OntoItem;
	}
}

void SDataHierarchyEditor::Construct(const FArguments& InArgs, TObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel)
{
	// A UDataHierarchyViewModelBase accepts at most one SDataHierarchyEditor consumer at a time via single-cast delegates.
	// A view model can legitimately exist with no widget bound to it -- that is fine. What is not fine is two widgets
	// competing for the same view model. If one of our delegates is already bound when we arrive, a prior widget is
	// still alive: its destructor hasn't run, so its bindings haven't been torn down. Constructing a second widget
	// against the same view model here would silently overwrite the first widget's bindings and leave it orphaned.
	// The caller should reuse the already-summoned widget (e.g. just bring its window to front) instead,
	// or ensure they don't keep the widget alive unnecessarily.
	ensureMsgf(!InHierarchyViewModel->OnRefreshHierarchyView().IsBound(),
		TEXT("SDataHierarchyEditor::Construct: a widget is already bound to this UDataHierarchyViewModelBase. "
			 "Only one widget can consume a given view model at a time; the prior widget must be destroyed before "
			 "a second is constructed. Summoning code should detect the live widget and reuse it rather than "
			 "constructing a second SDataHierarchyEditor for the same view model."));

	HierarchyViewModel = InHierarchyViewModel;

	// If the user hasn't called initialize themselves, we do it here, but ideally the user should do it themselves where appropriate
	if(HierarchyViewModel->IsInitialized() == false)
	{
		HierarchyViewModel->Initialize();
	}

	if(HierarchyViewModel->SupportsSourcePanel())
	{
		SourceRoot.Reset(NewObject<UHierarchyRoot>(HierarchyViewModel->GetOuterForSourceRoot(), MakeUniqueObjectName(HierarchyViewModel->GetOuterForSourceRoot(), UHierarchyRoot::StaticClass()), RF_Transient));
		TSharedPtr<FHierarchyElementViewModel> ViewModel = HierarchyViewModel->CreateViewModelForElement(SourceRoot.Get(), nullptr);
		SourceRootViewModel = StaticCastSharedPtr<FHierarchyRootViewModel>(ViewModel);	
		SourceRootViewModel->Initialize();
		SourceRootViewModel->AddChildFilter(FHierarchyElementViewModel::FOnFilterChild::CreateSP(this, &SDataHierarchyEditor::FilterForSourceSection));
		SourceRootViewModel->OnSyncPropagated().BindSP(this, &SDataHierarchyEditor::RequestRefreshSourceViewNextFrame, false);
		SourceRootViewModel->OnSectionsChanged().BindSP(this, &SDataHierarchyEditor::RefreshSectionsView);

		AllSourceSection.Reset(NewObject<UHierarchySection>(HierarchyViewModel.Get(), HierarchyViewModel->GetSectionDataClass(), UDataHierarchyViewModelBase::AllSectionSourceObjectName));
		DefaultSourceSectionViewModel = MakeShared<FHierarchySectionViewModel>(AllSourceSection.Get(), SourceRootViewModel.ToSharedRef(), HierarchyViewModel);
		DefaultSourceSectionViewModel->SetSectionName("All");
	}

	HierarchyViewModel->OnInitialized().BindSP(this, &SDataHierarchyEditor::Reinitialize);
	HierarchyViewModel->OnNavigateToElementIdentityInHierarchyRequested().BindSP(this, &SDataHierarchyEditor::NavigateToHierarchyElement);
	HierarchyViewModel->OnNavigateToElementInHierarchyRequested().BindSP(this, &SDataHierarchyEditor::NavigateToHierarchyElement);
	HierarchyViewModel->OnRefreshSourceItemsRequested().BindSP(this, &SDataHierarchyEditor::RefreshSourceItems);
	HierarchyViewModel->OnRefreshViewRequested().BindSP(this, &SDataHierarchyEditor::RefreshAllViews);
	HierarchyViewModel->OnRefreshSourceView().BindSP(this, &SDataHierarchyEditor::RefreshSourceView);
	HierarchyViewModel->OnRefreshHierarchyView().BindSP(this, &SDataHierarchyEditor::RefreshHierarchyView);
	HierarchyViewModel->OnRefreshSectionsView().BindSP(this, &SDataHierarchyEditor::RefreshSectionsView);
	HierarchyViewModel->OnHierarchySectionActivated().BindSP(this, &SDataHierarchyEditor::OnHierarchySectionActivated);

	BindToHierarchyRootViewModel();
	
	OnGenerateRowContentWidget = InArgs._OnGenerateRowContentWidget;
	OnGenerateCustomDetailsPanelNameWidget = InArgs._OnGenerateCustomDetailsPanelNameWidget;
	
	if(!ensureMsgf(OnGenerateRowContentWidget.IsBound(), TEXT("Please add a function binding to the OnGenerateRowContentWidget slate event. Using default row content.")))
	{
		OnGenerateRowContentWidget = FOnGenerateRowContentWidget::CreateLambda([](TSharedRef<FHierarchyElementViewModel> HierarchyElement) -> TSharedRef<SWidget>
		{
			if(HierarchyElement->GetDataMutable()->IsA<UHierarchyCategory>())
			{
				TSharedRef<FHierarchyCategoryViewModel> TreeViewCategory = StaticCastSharedRef<FHierarchyCategoryViewModel>(HierarchyElement);
				return SNew(SHierarchyElement, TreeViewCategory)
					.Style(&FDataHierarchyEditorStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("HierarchyEditor.Category"));
			}

			if(HierarchyElement->GetDataMutable()->IsA<UHierarchyItem>())
			{
				TSharedRef<FHierarchyItemViewModel> TreeViewItem = StaticCastSharedRef<FHierarchyItemViewModel>(HierarchyElement);
				return SNew(SHierarchyElement, TreeViewItem);
			}
			
			return SNew(STextBlock).Text(HierarchyElement->ToStringAsText());
		});
	}

	SSplitter::FSlot* SourcePanelSlot = nullptr;
	SSplitter::FSlot* DetailsPanelSlot = nullptr;

	TSharedRef<SWidget> AddSectionButton = SNew(SButton)
		.OnClicked(this, &SDataHierarchyEditor::OnAddSectionClicked)
		.ButtonStyle(FDataHierarchyEditorStyle::Get(), "HierarchyEditor.ButtonStyle")
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[			
				SNew(STextBlock)
				.Text(LOCTEXT("AddSectionLabel","Add Section"))				
			]
		];

	TSharedRef<SWidget> AddCategoryButton = SNew(SButton)
		.Visibility(HierarchyViewModel->GetCategoryDataClass() ? EVisibility::Visible : EVisibility::Collapsed)
		.OnClicked(this, &SDataHierarchyEditor::OnAddCategoryClicked)
		.ButtonStyle(FDataHierarchyEditorStyle::Get(), "HierarchyEditor.ButtonStyle")
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[			
				SNew(STextBlock)
				.Text(LOCTEXT("AddCategoryLabel","Add Category"))				
			]
		];

	TSharedRef<SWidget> AddCustomButton = SNew(SPositiveActionButton)
		.Visibility_Lambda([this]() -> EVisibility
		{
			return HierarchyViewModel->GetAdditionalTypesToAddInUi().Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
		})
		.Text(LOCTEXT("AddNewButtonLabel", "Add"))
		.OnGetMenuContent(this, &SDataHierarchyEditor::OnGetCustomAddContent);
	
	ChildSlot
	[
		SNew(SBorder)
		.Padding(0.f)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			.PhysicalSplitterHandleSize(2.f)
			+ SSplitter::Slot()
			.Expose(SourcePanelSlot)
			.Value(0.3f)
			.MinSize(0.1f)
			+ SSplitter::Slot()
			.Value(0.4f)
			.MinSize(0.1f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SVerticalBox)
					.Visibility(HierarchyViewModel->GetSectionDataClass() ? EVisibility::Visible : EVisibility::Collapsed)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBorder)
						.Padding(0.f)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
						.HAlign(HAlign_Left)
						[
							AddSectionButton
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(1.f)
					[
						SAssignNew(HierarchySectionBox, SWrapBox)
						.UseAllottedSize(true)
					]				
				]
				+ SVerticalBox::Slot()
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBorder)
						.Padding(0.f)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Left)
							.AutoWidth()
							[
								AddCategoryButton
							]
							+ SHorizontalBox::Slot()
							[
								SNew(SSpacer)
							]
							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Right)
							.AutoWidth()
							[
								AddCustomButton
							]
						]
					]
					+ SVerticalBox::Slot()
					.FillHeight(0.1f)
					.Padding(1.f, 4.f, 1.f, 0.f)
					[
						SNew(SDropTarget)
						.OnDropped(this, &SDataHierarchyEditor::HandleHierarchyRootDrop)
						.OnAllowDrop(this, &SDataHierarchyEditor::OnCanDropOnRoot)
						.OnDragEnter(this, &SDataHierarchyEditor::OnRootDragEnter)
						.OnDragLeave(this, &SDataHierarchyEditor::OnRootDragLeave)
						[
							SNew(SBorder)
							.Padding(0.f)
							.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
							[
								SNew(SBorder)
								.Padding(1.f)
								.BorderImage(FAppStyle::GetBrush("DashedBorder"))
								.BorderBackgroundColor(FLinearColor(0.2f, 0.2f, 0.2f, 0.5f))
								[
									SNew(SBox)
									.HAlign(HAlign_Center)
									.VAlign(VAlign_Center)
									[
										SNew(SImage)
										.Image(FDataHierarchyEditorStyle::Get().GetBrush("HierarchyEditor.RootDropIcon"))
										.ColorAndOpacity(this, &SDataHierarchyEditor::GetRootIconColor)
									]
								]
							]
						]
					]
					+ SVerticalBox::Slot()
					.Padding(1.f, 0.f)
					[
						SAssignNew(HierarchyTreeView, STreeView<TSharedPtr<FHierarchyElementViewModel>>)
						.SelectionMode(InHierarchyViewModel->GetHierarchySelectionMode())
						.TreeItemsSource(&InHierarchyViewModel->GetHierarchyItems())
						.OnSelectionChanged(this, &SDataHierarchyEditor::OnSelectionChanged)
						.OnGenerateRow(this, &SDataHierarchyEditor::GenerateHierarchyItemRow)
						.OnGetChildren_UObject(InHierarchyViewModel.Get(), &UDataHierarchyViewModelBase::OnGetChildren)
						.OnItemToString_Debug_UObject(InHierarchyViewModel.Get(), &UDataHierarchyViewModelBase::OnElementToStringDebug)
						.OnContextMenuOpening(this, &SDataHierarchyEditor::SummonContextMenuForSelectedRows, true)
					]
				]
			]
			+ SSplitter::Slot()
			.Expose(DetailsPanelSlot)		
			.Value(0.3f)
			.MinSize(0.1f)
		]
	];

	if(InHierarchyViewModel->SupportsSourcePanel())
	{
		TSharedRef<SWidget> SourcePanelContent = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.f)
		[
			SAssignNew(SourceSearchBox, SSearchBox)
			.OnTextChanged(this, &SDataHierarchyEditor::OnSourceSearchTextChanged)
			.OnTextCommitted(this, &SDataHierarchyEditor::OnSourceSearchTextCommitted)
			.OnSearch(SSearchBox::FOnSearch::CreateSP(this, &SDataHierarchyEditor::OnSearchButtonClicked))
			.DelayChangeNotificationsWhileTyping(true)
			.SearchResultData(this, &SDataHierarchyEditor::GetSearchResultData)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(2.f)
		[
			SAssignNew(SourceSectionBox, SWrapBox)
			.UseAllottedSize(true)
		]
		+ SVerticalBox::Slot()
		.Padding(1.f, 2.f)
		[
			SAssignNew(SourceTreeView, STreeView<TSharedPtr<FHierarchyElementViewModel>>)
			.SelectionMode(ESelectionMode::Multi)
			.TreeItemsSource(&GetSourceItems())
			.OnSelectionChanged(this, &SDataHierarchyEditor::OnSelectionChanged)
			.OnGenerateRow(this, &SDataHierarchyEditor::GenerateSourceItemRow)
			.OnGetChildren_UObject(InHierarchyViewModel.Get(), &UDataHierarchyViewModelBase::OnGetChildren)
			.OnItemToString_Debug_UObject(InHierarchyViewModel.Get(), &UDataHierarchyViewModelBase::OnElementToStringDebug)
			.OnContextMenuOpening(this, &SDataHierarchyEditor::SummonContextMenuForSelectedRows, false)
		];

		SourcePanelSlot->AttachWidget(SourcePanelContent);
	}
	else
	{
		SourcePanelSlot->SetMinSize(0.f);
		SourcePanelSlot->SetSizeValue(0.f);
		SourcePanelSlot->SetResizable(false);
	}
	
	if(InHierarchyViewModel->SupportsDetailsPanel())
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::ObjectsUseNameArea;
		DetailsViewArgs.bShowObjectLabel = false;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NotifyHook = this;

		DetailsPanel = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		
		if(OnGenerateCustomDetailsPanelNameWidget.IsBound())
		{
			TSharedRef<SWidget> CustomDetailsPanelNameWidget = OnGenerateCustomDetailsPanelNameWidget.Execute(nullptr);
			DetailsPanel->SetNameAreaCustomContent(CustomDetailsPanelNameWidget);
		}

		DetailsPanel->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &SDataHierarchyEditor::IsDetailsPanelEditingAllowed));
		
		for(auto& Customizations : HierarchyViewModel->GetInstanceCustomizations())
		{
			DetailsPanel->RegisterInstancedCustomPropertyLayout(Customizations.Key, Customizations.Value);
		}

		OnSelectionChanged(nullptr, ESelectInfo::Type::Direct);
		
		DetailsPanelSlot->AttachWidget(DetailsPanel.ToSharedRef());
	}

	// For shortcuts, we use the commandlist logic which lacks context information (such as 'what IS the selected item')
	
	HierarchyViewModel->GetCommands()->MapAction(FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SDataHierarchyEditor::RequestRenameSelectedItem),
		FCanExecuteAction::CreateSP(this, &SDataHierarchyEditor::CanRequestRenameSelectedItem),
		FIsActionChecked(), FIsActionButtonVisible::CreateSP(this, &SDataHierarchyEditor::CanRequestRenameSelectedItem));
	
	HierarchyViewModel->GetCommands()->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SDataHierarchyEditor::DeleteSelectedHierarchyItems),
		FCanExecuteAction::CreateSP(this, &SDataHierarchyEditor::CanDeleteSelectedElements),
		FIsActionChecked(), FIsActionButtonVisible::CreateSP(this, &SDataHierarchyEditor::CanDeleteSelectedElements));
	
	HierarchyViewModel->GetCommands()->MapAction(FDataHierarchyEditorCommands::Get().FindInHierarchy,
		FExecuteAction::CreateSP(this, &SDataHierarchyEditor::NavigateToMatchingHierarchyElementFromSelectedSourceElement),
		FCanExecuteAction::CreateSP(this, &SDataHierarchyEditor::CanNavigateToMatchingHierarchyElementFromSelectedSourceElement),
		FIsActionChecked(), FIsActionButtonVisible::CreateSP(this, &SDataHierarchyEditor::CanNavigateToMatchingHierarchyElementFromSelectedSourceElement));
	
	HierarchyViewModel->ForceFullRefresh();
	
	SetActiveSourceSection(DefaultSourceSectionViewModel);
}

SDataHierarchyEditor::~SDataHierarchyEditor()
{
	SourceSearchResults.Empty();
	FocusedSearchResult.Reset();
	
	ClearSourceItems();
	
	if(HierarchyViewModel.IsValid())
	{
		HierarchyViewModel->OnInitialized().Unbind();
		HierarchyViewModel->OnNavigateToElementIdentityInHierarchyRequested().Unbind();
		HierarchyViewModel->OnNavigateToElementInHierarchyRequested().Unbind();
		HierarchyViewModel->OnRefreshSourceItemsRequested().Unbind();
		HierarchyViewModel->OnRefreshViewRequested().Unbind();
		HierarchyViewModel->OnRefreshSourceView().Unbind();
		HierarchyViewModel->OnRefreshHierarchyView().Unbind();
		HierarchyViewModel->OnRefreshSectionsView().Unbind();
		HierarchyViewModel->OnHierarchySectionActivated().Unbind();

		UnbindFromHierarchyRootViewModel();

		HierarchyViewModel->GetCommands()->UnmapAction(FGenericCommands::Get().Delete);
		HierarchyViewModel->GetCommands()->UnmapAction(FGenericCommands::Get().Rename);
		HierarchyViewModel->GetCommands()->UnmapAction(FDataHierarchyEditorCommands::Get().FindInHierarchy);
	}

	if(HierarchyViewModel->SupportsSourcePanel())
	{
		SourceRootViewModel->OnSyncPropagated().Unbind();
		SourceRootViewModel->OnSectionsChanged().Unbind();
		SourceRootViewModel.Reset();
		SourceRoot->ConditionalBeginDestroy();
		SourceRoot = nullptr;
	}
}

void SDataHierarchyEditor::RefreshSourceItems()
{
	if(HierarchyViewModel->SupportsSourcePanel())
	{
		SourceRoot->EmptyAllData();
		HierarchyViewModel->PrepareSourceItems(SourceRoot.Get(), SourceRootViewModel);
		SourceRootViewModel->SyncViewModelsToData();
		RefreshSourceView(false);
		RefreshSectionsView();
	}
}

void SDataHierarchyEditor::RefreshAllViews(bool bFullRefresh)
{
	RefreshSourceView(bFullRefresh);
	RefreshHierarchyView(bFullRefresh);
	RefreshSectionsView();
}

void SDataHierarchyEditor::RequestRefreshAllViewsNextFrame(bool bFullRefresh)
{
	RequestRefreshSourceViewNextFrame(bFullRefresh);
	RequestRefreshHierarchyViewNextFrame(bFullRefresh);
	RequestRefreshSectionsViewNextFrame();
}

FReply SDataHierarchyEditor::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	UHierarchyMenuContext* HierarchyMenuContext = NewObject<UHierarchyMenuContext>();
	HierarchyMenuContext->HierarchyViewModel = HierarchyViewModel;
	
	FToolMenuContext Context;
	Context.AddObject(HierarchyMenuContext);
	
	UToolMenus::Get()->GenerateMenu(HierarchyViewModel->GetContextMenuName(), Context);
	return HierarchyViewModel->GetCommands()->ProcessCommandBindings(InKeyEvent) ? FReply::Handled() : FReply::Unhandled();
}

FReply SDataHierarchyEditor::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// we catch any mouse button down event so that we can continue using our commands
	return FReply::Handled().SetUserFocus(AsShared(), EFocusCause::Mouse, true);
}

FReply SDataHierarchyEditor::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Handled().SetUserFocus(AsShared(), EFocusCause::Mouse, true);
}

/** Sorts items to match their display order in the tree by doing a depth-first walk of the root's filtered children. */
static void SortByTreeDisplayOrder(TArray<TSharedPtr<FHierarchyElementViewModel>>& Items, TSharedPtr<FHierarchyRootViewModel> Root)
{
	if(Items.Num() <= 1 || !Root.IsValid())
	{
		return;
	}

	TSet<TSharedPtr<FHierarchyElementViewModel>> ItemSet(Items);
	TArray<TSharedPtr<FHierarchyElementViewModel>> Sorted;
	Sorted.Reserve(Items.Num());

	// Depth-first traversal; collect items in display order
	TArray<TSharedPtr<FHierarchyElementViewModel>> Stack;
	const TArray<TSharedPtr<FHierarchyElementViewModel>>& RootChildren = Root->GetFilteredChildren();
	for(int32 i = RootChildren.Num() - 1; i >= 0; --i)
	{
		Stack.Add(RootChildren[i]);
	}

	while(Stack.Num() > 0)
	{
		TSharedPtr<FHierarchyElementViewModel> Current = Stack.Pop();
		if(ItemSet.Contains(Current))
		{
			Sorted.Add(Current);
			if(Sorted.Num() == Items.Num())
			{
				break;
			}
		}

		const TArray<TSharedPtr<FHierarchyElementViewModel>>& Children = Current->GetFilteredChildren();
		for(int32 i = Children.Num() - 1; i >= 0; --i)
		{
			Stack.Add(Children[i]);
		}
	}

	Items = MoveTemp(Sorted);
}

FReply SDataHierarchyEditor::OnHierarchyRowDragDetected(TSharedRef<FHierarchyElementViewModel> DraggedRow, const FGeometry& Geometry, const FPointerEvent& PointerEvent)
{
	TArray<TSharedPtr<FHierarchyElementViewModel>> SelectedItems = HierarchyTreeView->GetSelectedItems();

	// If the dragged row is not in the selection, treat as a single-item drag
	if(!SelectedItems.Contains(DraggedRow))
	{
		SelectedItems = {DraggedRow};
	}

	SortByTreeDisplayOrder(SelectedItems, HierarchyViewModel->GetHierarchyRootViewModel());
	return HandleMultiDragDetected(SelectedItems, false);
}

FReply SDataHierarchyEditor::OnSourceRowDragDetected(TSharedRef<FHierarchyElementViewModel> DraggedRow, const FGeometry& Geometry, const FPointerEvent& PointerEvent)
{
	TArray<TSharedPtr<FHierarchyElementViewModel>> SelectedItems = SourceTreeView->GetSelectedItems();

	// If the dragged row is not in the selection, treat as a single-item drag
	if(!SelectedItems.Contains(DraggedRow))
	{
		SelectedItems = {DraggedRow};
	}

	SortByTreeDisplayOrder(SelectedItems, SourceRootViewModel);
	return HandleMultiDragDetected(SelectedItems, true);
}

FReply SDataHierarchyEditor::HandleMultiDragDetected(const TArray<TSharedPtr<FHierarchyElementViewModel>>& ItemsToDrag, bool bIsSource)
{
	if(ItemsToDrag.IsEmpty())
	{
		return FReply::Unhandled();
	}

	// Validate all items can be dragged (all-or-nothing)
	for(const TSharedPtr<FHierarchyElementViewModel>& Item : ItemsToDrag)
	{
		FHierarchyElementViewModel::FResultWithUserFeedback CanDragResult = Item->CanDrag();
		if(CanDragResult == false)
		{
			if(CanDragResult.UserFeedback.IsSet())
			{
				FNotificationInfo CantDragInfo(CanDragResult.UserFeedback.GetValue());
				FSlateNotificationManager::Get().AddNotification(CantDragInfo);
			}
			return FReply::Unhandled();
		}
	}

	// For source items, validate none already exist in the hierarchy
	if(bIsSource)
	{
		for(const TSharedPtr<FHierarchyElementViewModel>& Item : ItemsToDrag)
		{
			if(HierarchyViewModel->GetHierarchyRootViewModel()->FindViewModelForChild(Item->GetData()->GetPersistentIdentity(), true) != nullptr)
			{
				FNotificationInfo Info(LOCTEXT("MultiDragAlreadyInHierarchy", "One or more selected items already exist in the hierarchy."));
				FSlateNotificationManager::Get().AddNotification(Info);
				return FReply::Unhandled();
			}

			// Also check children of each item
			TArray<TSharedPtr<FHierarchyElementViewModel>> AllChildren;
			Item->GetChildrenViewModelsForType<UHierarchyElement, FHierarchyElementViewModel>(AllChildren, true);
			for(const TSharedPtr<FHierarchyElementViewModel>& ChildViewModel : AllChildren)
			{
				if(HierarchyViewModel->GetHierarchyRootViewModel()->FindViewModelForChild(ChildViewModel->GetData()->GetPersistentIdentity(), true) != nullptr)
				{
					FNotificationInfo Info(LOCTEXT("MultiDragChildAlreadyInHierarchy", "One or more selected items contain children that already exist in the hierarchy."));
					FSlateNotificationManager::Get().AddNotification(Info);
					return FReply::Unhandled();
				}
			}
		}
	}

	// Compute hierarchy containment flags
	TArray<FDraggedElementEntry> DraggedEntries;
	DraggedEntries.Reserve(ItemsToDrag.Num());
	for(const TSharedPtr<FHierarchyElementViewModel>& Item : ItemsToDrag)
	{
		FDraggedElementEntry Entry;
		Entry.Element = Item;
		Entry.bIsContainedInAnotherDraggedElement = false;

		for(const TSharedPtr<FHierarchyElementViewModel>& OtherItem : ItemsToDrag)
		{
			if(Item != OtherItem && Item->HasParent(OtherItem, true))
			{
				Entry.bIsContainedInAnotherDraggedElement = true;
				break;
			}
		}

		DraggedEntries.Add(Entry);
	}

	FDataHierarchyDragDropContext DragContext(MoveTemp(DraggedEntries));
	TSharedRef<FHierarchyDragDropOp> DragDropOp = HierarchyViewModel->CreateDragDropOp(DragContext);
	DragDropOp->SetFromSourceList(bIsSource);

	return FReply::Handled().BeginDragDrop(DragDropOp);
}

FReply SDataHierarchyEditor::OnAddCategoryClicked() const
{
	TArray<TSharedPtr<FHierarchyElementViewModel>> SelectedItems;
	HierarchyTreeView->GetSelectedItems(SelectedItems);

	// We can only add categories under categories or the root
	if(SelectedItems.Num() == 1 && SelectedItems[0]->GetData()->IsA<UHierarchyCategory>())
	{
		HierarchyViewModel.Get()->AddCategory(SelectedItems[0]);
	}
	else
	{
		HierarchyViewModel.Get()->AddCategory(nullptr);
	}
	
	return FReply::Handled();
}

FReply SDataHierarchyEditor::OnAddSectionClicked() const
{
	HierarchyViewModel.Get()->GetHierarchyRootViewModel()->AddSection();
	return FReply::Handled();
}

TSharedPtr<SWidget> SDataHierarchyEditor::SummonContextMenuForSelectedRows(bool bFromHierarchy) const
{
	TArray<TSharedPtr<FHierarchyElementViewModel>> ViewModels;
	if(bFromHierarchy)
	{
		HierarchyTreeView->GetSelectedItems(ViewModels);
	}
	else
	{
		SourceTreeView->GetSelectedItems(ViewModels);
	}

	if(ViewModels.IsEmpty())
	{
		return nullptr;
	}
	
	// In the hierarchy view, if there is no source view, items will no have anything in their context menu.
	const bool bOneViewModelIsItem = Algo::AnyOf(ViewModels, [](const TSharedPtr<FHierarchyElementViewModel>& ElementView) -> bool
	{
		if (const UHierarchyElement* HierachyElementData = ElementView && ElementView.IsValid() ? ElementView->GetData() : nullptr)
		{
			return HierachyElementData->IsA<UHierarchyItem>();
		}
		return false;
	});
	if (bFromHierarchy && !SourceTreeView && bOneViewModelIsItem)
	{
		return nullptr;
	}

	return SummonContextMenu(ViewModels);
}

void SDataHierarchyEditor::RefreshSourceView(bool bFullRefresh) const
{
	if(HierarchyViewModel->SupportsSourcePanel() == false)
	{
		return;
	}
	
	SourceTreeView->SetTreeItemsSource(&GetSourceItems());
	if(bFullRefresh)
	{
		SourceTreeView->RebuildList();
	}
	else
	{
		SourceTreeView->RequestTreeRefresh();
	}
	
	ExpandEntriesByDefault(SourceTreeView);
}

void SDataHierarchyEditor::RequestRefreshSourceViewNextFrame(bool bFullRefresh)
{
	if(HierarchyViewModel->SupportsSourcePanel() == false)
	{
		return;
	}
	
	if(!RefreshSourceViewNextFrameHandle.IsValid())
	{
		RefreshSourceViewNextFrameHandle = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this, bFullRefresh](double CurrentTime, float DeltaTime)
		{
			RefreshSourceView(bFullRefresh);
			RefreshSourceViewNextFrameHandle.Reset();
			return EActiveTimerReturnType::Stop;
		}));
	}
}

void SDataHierarchyEditor::RefreshHierarchyView(bool bFullRefresh) const
{
	// Save expansion state before refreshing so we can restore it after
	SaveExpansionState(HierarchyTreeView);

	// the top layer objects might have changed due to filtering. We need to refresh these too.
	HierarchyTreeView->SetTreeItemsSource(&HierarchyViewModel->GetHierarchyItems());
	if(bFullRefresh)
	{
		HierarchyTreeView->RebuildList();
	}
	else
	{
		HierarchyTreeView->RequestTreeRefresh();
	}

	RestoreExpansionState(HierarchyTreeView);
}

void SDataHierarchyEditor::RequestRefreshHierarchyViewNextFrame(bool bFullRefresh)
{
	if(!RefreshHierarchyViewNextFrameHandle.IsValid())
	{
		RefreshHierarchyViewNextFrameHandle = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this, bFullRefresh](double CurrentTime, float DeltaTime)
		{
			RefreshHierarchyView(bFullRefresh);
			RefreshHierarchyViewNextFrameHandle.Reset();
			return EActiveTimerReturnType::Stop;
		}));
	}
}

void SDataHierarchyEditor::RefreshSectionsView()
{
	if(HierarchyViewModel->SupportsSourcePanel())
	{
		SourceSectionBox->ClearChildren();

		for (TSharedPtr<FHierarchySectionViewModel>& SourceSection : SourceRootViewModel->GetSectionViewModels())
		{
			TSharedPtr<SHierarchySection> SectionWidget = SNew(SHierarchySection, SourceSection)
			.IsSectionActive_Lambda([this, SourceSection]()
			{
				return GetActiveSourceSection() == SourceSection ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnSectionActivated_Lambda([this](TSharedPtr<FHierarchySectionViewModel> SectionViewModel)
			{
				SetActiveSourceSection(SectionViewModel);
			});
			
			SourceSectionBox->AddSlot()
			.Padding(2.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Fill)
			[
				SectionWidget.ToSharedRef()		
			];
		}

		if(SourceRootViewModel->GetSectionViewModels().Num() > 0)
		{			
			TSharedPtr<SHierarchySection> DefaultSourceSection = SNew(SHierarchySection, DefaultSourceSectionViewModel)
			.IsSectionActive_Lambda([this]()
			{
				return GetActiveSourceSection() == DefaultSourceSectionViewModel ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnSectionActivated(this, &SDataHierarchyEditor::SetActiveSourceSection);

			SourceSectionBox->AddSlot()
			.Padding(2.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Fill)
			[
				DefaultSourceSection.ToSharedRef()
			];
		}
	}

	HierarchySectionBox->ClearChildren();
	for (const TSharedPtr<FHierarchySectionViewModel>& HierarchySection : HierarchyViewModel->GetHierarchyRootViewModel()->GetSectionViewModels())
	{
		TSharedPtr<SHierarchySection> SectionWidget = SNew(SHierarchySection, HierarchySection)
		.IsSectionActive_Lambda([this, HierarchySection]()
		{
			return HierarchyViewModel->GetActiveHierarchySectionViewModel() == HierarchySection ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		})
		.OnSectionActivated_Lambda([this](TSharedPtr<FHierarchySectionViewModel> SectionViewModel)
		{
			HierarchyViewModel->SetActiveHierarchySection(SectionViewModel);
		});
		
		HierarchySectionBox->AddSlot()
		.Padding(2.f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Fill)
		[
			SectionWidget.ToSharedRef()		
		];
	}
	
	TSharedPtr<SHierarchySection> DefaultHierarchySection = SNew(SHierarchySection, HierarchyViewModel->GetDefaultHierarchySectionViewModel())
	.IsSectionActive_Lambda([this]()
	{
		return HierarchyViewModel->GetActiveHierarchySectionViewModel() == HierarchyViewModel->GetDefaultHierarchySectionViewModel() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	})
	.OnSectionActivated_Lambda([this](TSharedPtr<FHierarchySectionViewModel> SectionViewModel)
	{
		HierarchyViewModel->SetActiveHierarchySection(SectionViewModel);
	});

	HierarchySectionBox->AddSlot()
	.Padding(2.f)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Fill)
	[
		DefaultHierarchySection.ToSharedRef()
	];
}

void SDataHierarchyEditor::RequestRefreshSectionsViewNextFrame()
{
	if(!RefreshSectionsViewNextFrameHandle.IsValid())
	{
		RefreshSectionsViewNextFrameHandle = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateLambda([this](double CurrentTime, float DeltaTime)
		{
			RefreshSectionsView();
			RefreshSectionsViewNextFrameHandle.Reset();
			return EActiveTimerReturnType::Stop;
		}));
	}
}

void SDataHierarchyEditor::NavigateToHierarchyElement(FHierarchyElementIdentity Identity) const
{
	if(TSharedPtr<FHierarchyElementViewModel> ViewModel = HierarchyViewModel->GetHierarchyRootViewModel()->FindViewModelForChild(Identity, true))
	{
		NavigateToHierarchyElement(ViewModel);
	}
}

void SDataHierarchyEditor::NavigateToHierarchyElement(TSharedPtr<FHierarchyElementViewModel> Item) const
{
	TArray<TSharedPtr<FHierarchyElementViewModel>> ParentChain;
	for(TWeakPtr<FHierarchyElementViewModel> Parent = Item->GetParent(); Parent.IsValid(); Parent = Parent.Pin()->GetParent())
	{
		ParentChain.Add(Parent.Pin());
	}

	for(int32 ParentIndex = ParentChain.Num()-1; ParentIndex >= 0; ParentIndex--)
	{
		HierarchyTreeView->SetItemExpansion(ParentChain[ParentIndex], true);
	}
		
	HierarchyTreeView->SetSelection(Item);
	HierarchyTreeView->RequestScrollIntoView(Item);
}

bool SDataHierarchyEditor::IsItemSelected(TSharedPtr<FHierarchyElementViewModel> Item) const
{
	return HierarchyTreeView->IsItemSelected(Item);
}

TSharedRef<ITableRow> SDataHierarchyEditor::GenerateSourceItemRow(TSharedPtr<FHierarchyElementViewModel> HierarchyItem, const TSharedRef<STableViewBase>& TableViewBase)
{
	return SNew(STableRow<TSharedPtr<FHierarchyElementViewModel>>, TableViewBase)
	.Style(HierarchyItem->GetRowStyle() ? HierarchyItem->GetRowStyle() : &FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
	.OnDragDetected_Lambda([this, HierarchyItem](const FGeometry& Geometry, const FPointerEvent& PointerEvent) { return OnSourceRowDragDetected(HierarchyItem.ToSharedRef(), Geometry, PointerEvent); })
	.Padding(FMargin(2))
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1.f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.HeightOverride(10.f)
			.WidthOverride(10.f)
			.Visibility_Lambda([HierarchyItem, this]()
			{
				TArray<TSharedPtr<FHierarchyElementViewModel>> AllChildren;
				HierarchyItem->GetChildrenViewModelsForType<UHierarchyElement, FHierarchyElementViewModel>(AllChildren, true);

				bool bCanDrag = HierarchyItem->GetHierarchyViewModel()->GetHierarchyRootViewModel()->FindViewModelForChild(HierarchyItem->GetData()->GetPersistentIdentity(), true) == nullptr;			

				if(bCanDrag)
				{
					for(TSharedPtr<FHierarchyElementViewModel>& ItemViewModel : AllChildren)
					{
						if(HierarchyItem->GetHierarchyViewModel()->GetHierarchyRootViewModel()->FindViewModelForChild(ItemViewModel->GetData()->GetPersistentIdentity(), true) != nullptr)
						{
							bCanDrag = false;
							break;
						}
					}
				}

				return bCanDrag ? EVisibility::Collapsed : EVisibility::Visible;
			})
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Lock"))
				.ToolTipText(LOCTEXT("CantDragItemAlreadyInHierarchyTooltip", "This item already exists within the hierarchy and can not be dragged. Drag the existing one within the hierarchy directly."))
			]
		]
		+ SHorizontalBox::Slot()
		[
			OnGenerateRowContentWidget.Execute(HierarchyItem.ToSharedRef())
		]
	];
}

TSharedRef<ITableRow> SDataHierarchyEditor::GenerateHierarchyItemRow(TSharedPtr<FHierarchyElementViewModel> HierarchyItem, const TSharedRef<STableViewBase>& TableViewBase)
{
	return SNew(STableRow<TSharedPtr<FHierarchyElementViewModel>>, TableViewBase)
	.Style(HierarchyItem->GetRowStyle() ? HierarchyItem->GetRowStyle() : &FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
	.OnAcceptDrop(HierarchyItem.ToSharedRef(), &FHierarchyElementViewModel::OnDroppedOnRow)
	.OnCanAcceptDrop(HierarchyItem.ToSharedRef(), &FHierarchyElementViewModel::OnCanRowAcceptDrop)
	.OnDragDetected_Lambda([this, HierarchyItem](const FGeometry& Geometry, const FPointerEvent& PointerEvent) { return OnHierarchyRowDragDetected(HierarchyItem.ToSharedRef(), Geometry, PointerEvent); })
	.OnDragLeave(HierarchyItem.ToSharedRef(), &FHierarchyElementViewModel::OnRowDragLeave)
	.Padding(FMargin(2))
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1.f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.HeightOverride(10.f)
			.WidthOverride(10.f)
			.Visibility_Lambda([HierarchyItem]()
			{
				return HierarchyItem->IsEditableByUser().bResult ? EVisibility::Collapsed : EVisibility::Visible;
			})
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Lock"))
				.ToolTipText_Lambda([HierarchyItem]()
				{
					FHierarchyElementViewModel::FResultWithUserFeedback IsEditableResults = HierarchyItem->IsEditableByUser();
					if(IsEditableResults.bResult == false)
					{
						return IsEditableResults.UserFeedback.Get(FText::GetEmpty());
					}

					return FText::GetEmpty();
				})
			]
		]
		+ SHorizontalBox::Slot()
		[
			OnGenerateRowContentWidget.Execute(HierarchyItem.ToSharedRef())
		]
	];
}

bool SDataHierarchyEditor::FilterForSourceSection(TSharedPtr<const FHierarchyElementViewModel> ViewModel) const
{
	if(ActiveSourceSection.IsValid())
	{
		// If the currently selected section data is the all section we let everything pass
		if(ActiveSourceSection.Pin()->IsAllSection())
		{
			return true;
		}

		// if not, we check against identical section data
		FDataHierarchyElementMetaData_SectionAssociation SectionAssociation = ViewModel->GetData()->FindMetaDataOfTypeOrDefault<FDataHierarchyElementMetaData_SectionAssociation>();		
		return ActiveSourceSection.Pin()->GetSectionData() == SectionAssociation.Section;
	}

	return true;
}

void SDataHierarchyEditor::Reinitialize()
{
	// the hierarchy root view model has been recreated if the view model reinitialized. Therefore we update the bindings.
	BindToHierarchyRootViewModel();
	RefreshSourceItems();
	RefreshAllViews(true);
}

void SDataHierarchyEditor::BindToHierarchyRootViewModel()
{
	HierarchyViewModel->GetHierarchyRootViewModel()->OnSyncPropagated().BindSP(this, &SDataHierarchyEditor::RequestRefreshHierarchyViewNextFrame, false);
	HierarchyViewModel->GetHierarchyRootViewModel()->OnSectionsChanged().BindSP(this, &SDataHierarchyEditor::RefreshSectionsView);
	HierarchyViewModel->GetHierarchyRootViewModel()->OnSectionAdded().BindSP(this, &SDataHierarchyEditor::OnHierarchySectionAdded);
	HierarchyViewModel->GetHierarchyRootViewModel()->OnSectionDeleted().BindSP(this, &SDataHierarchyEditor::OnHierarchySectionDeleted);
}

void SDataHierarchyEditor::UnbindFromHierarchyRootViewModel() const
{
	if(HierarchyViewModel->GetHierarchyRootViewModel().IsValid())
	{
		HierarchyViewModel->GetHierarchyRootViewModel()->OnSyncPropagated().Unbind();
		HierarchyViewModel->GetHierarchyRootViewModel()->OnSectionsChanged().Unbind();
		HierarchyViewModel->GetHierarchyRootViewModel()->OnSectionAdded().Unbind();
		HierarchyViewModel->GetHierarchyRootViewModel()->OnSectionDeleted().Unbind();
	}
}

const TArray<TSharedPtr<FHierarchyElementViewModel>>& SDataHierarchyEditor::GetSourceItems() const
{
	return SourceRootViewModel->GetFilteredChildren();
}

bool SDataHierarchyEditor::IsDetailsPanelEditingAllowed() const
{
	for(const TWeakPtr<FHierarchyElementViewModel>& WeakVM : SelectedDetailsPanelItemViewModels)
	{
		if(WeakVM.IsValid() && WeakVM.Pin()->IsEditableByUser().bResult)
		{
			return true;
		}
	}
	return false;
}

void SDataHierarchyEditor::RequestRenameSelectedItem()
{
	TArray<TSharedPtr<FHierarchyElementViewModel>> SelectedItems = HierarchyTreeView->GetSelectedItems();

	if(SelectedItems.Num() == 0)
	{
		TSharedPtr<FHierarchySectionViewModel> ActiveSection = HierarchyViewModel->GetActiveHierarchySectionViewModel();
		if(ActiveSection)
		{
			SelectedItems = { ActiveSection };
		}
	}
	
	if(SelectedItems.Num() == 1)
	{
		SelectedItems[0]->RequestRename();
	}
}

bool SDataHierarchyEditor::CanRequestRenameSelectedItem() const
{
	TArray<TSharedPtr<FHierarchyElementViewModel>> SelectedItems = HierarchyTreeView->GetSelectedItems();

	if(SelectedItems.Num() == 0)
	{
		TSharedPtr<FHierarchySectionViewModel> ActiveSection = HierarchyViewModel->GetActiveHierarchySectionViewModel();
		if(ActiveSection)
		{
			SelectedItems = { ActiveSection };
		}
	}
	
	if(SelectedItems.Num() == 1)
	{
		return SelectedItems[0]->CanRename();
	}

	return false;
}

void SDataHierarchyEditor::ClearSourceItems() const
{
	if(HierarchyViewModel->SupportsSourcePanel() == false)
	{
		return;
	}
	
	SourceRoot->GetChildrenMutable().Empty();
	SourceRoot->GetSectionDataMutable().Empty();
	SourceRootViewModel->GetChildrenMutable().Empty();
	SourceRootViewModel->GetSectionViewModels().Empty();
}

void SDataHierarchyEditor::DeleteItems(TArray<TSharedPtr<FHierarchyElementViewModel>> ItemsToDelete) const
{
	HierarchyViewModel->DeleteElements(ItemsToDelete);
}

void SDataHierarchyEditor::DeleteSelectedHierarchyItems() const
{	
	TArray<TSharedPtr<FHierarchyElementViewModel>> SelectedElements = HierarchyTreeView->GetSelectedItems();
	
	if(SelectedElements.Num() == 0)
	{
		TSharedPtr<FHierarchySectionViewModel> ActiveSection = HierarchyViewModel->GetActiveHierarchySectionViewModel();
		if(ActiveSection)
		{
			SelectedElements = { ActiveSection };
		}
	}
	
	DeleteItems(SelectedElements);
}

bool SDataHierarchyEditor::CanDeleteSelectedElements() const
{
	TArray<TSharedPtr<FHierarchyElementViewModel>> SelectedElements = HierarchyTreeView->GetSelectedItems();

	if(SelectedElements.Num() == 0)
	{
		TSharedPtr<FHierarchySectionViewModel> ActiveSection = HierarchyViewModel->GetActiveHierarchySectionViewModel();
		if(ActiveSection)
		{
			SelectedElements = { ActiveSection };
		}
	}
	
	if(SelectedElements.Num() > 0)
	{
		bool bCanDelete = true;
		for(TSharedPtr<FHierarchyElementViewModel> SelectedElement : SelectedElements)
		{
			bCanDelete &= SelectedElements[0]->CanDelete();
		}

		return bCanDelete;
	}

	return false;
}

void SDataHierarchyEditor::NavigateToMatchingHierarchyElementFromSelectedSourceElement() const
{
	TArray<TSharedPtr<FHierarchyElementViewModel>> SelectedElements = SourceTreeView->GetSelectedItems();

	if(SelectedElements.Num() != 1)
	{
		return;
	}

	TSharedPtr<FHierarchyElementViewModel> SelectedElement = SelectedElements[0];

	if(TSharedPtr<FHierarchyElementViewModel> MatchingViewModelInHierarchy = HierarchyViewModel->GetHierarchyRootViewModel()->FindViewModelForChild(SelectedElement->GetData()->GetPersistentIdentity(), true))
	{
		NavigateToHierarchyElement(MatchingViewModelInHierarchy.ToSharedRef());
	}
}

bool SDataHierarchyEditor::CanNavigateToMatchingHierarchyElementFromSelectedSourceElement() const
{
	if(HierarchyViewModel->SupportsSourcePanel() == false)
	{
		return false;
	}
	
	TArray<TSharedPtr<FHierarchyElementViewModel>> SelectedElements = SourceTreeView->GetSelectedItems();

	if(SelectedElements.Num() != 1)
	{
		return false;
	}

	TSharedPtr<FHierarchyElementViewModel> SelectedElement = SelectedElements[0];
	if(SelectedElement->IsForHierarchy())
	{
		return false;
	}
	
	if(TSharedPtr<FHierarchyElementViewModel> MatchingViewModelInHierarchy = HierarchyViewModel->GetHierarchyRootViewModel()->FindViewModelForChild(SelectedElement->GetData()->GetPersistentIdentity(), true))
	{
		return MatchingViewModelInHierarchy.IsValid();
	}
	
	return false;
}

void SDataHierarchyEditor::OnHierarchySectionActivated(TSharedPtr<FHierarchySectionViewModel> Section)
{
	// We forward nullptr in case this is the 'All' default section. We determine this by checking its data for validity.
	// The all section does not have an actual element associated with it.
	OnSelectionChanged(Section, ESelectInfo::Direct);
}

void SDataHierarchyEditor::OnSourceSectionActivated(TSharedPtr<FHierarchySectionViewModel> Section)
{
	OnSelectionChanged(Section, ESelectInfo::Direct);
	RunSourceSearch();
}

void SDataHierarchyEditor::OnHierarchySectionAdded(TSharedPtr<FHierarchySectionViewModel> AddedSection)
{
	HierarchyViewModel->SetActiveHierarchySection(AddedSection);
	AddedSection->RequestRenamePending();
}

void SDataHierarchyEditor::OnHierarchySectionDeleted(TSharedPtr<FHierarchySectionViewModel> DeletedSection)
{
	if(HierarchyViewModel->GetActiveHierarchySectionViewModel() == DeletedSection)
	{
		HierarchyViewModel->SetActiveHierarchySection(HierarchyViewModel->GetDefaultHierarchySectionViewModel());
	}
}

TSharedRef<SWidget> SDataHierarchyEditor::OnGetCustomAddContent()
{
	FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

	static FName MenuName = "DataHierarchyEditor.CustomAddMenu";
	if(UToolMenus::Get()->IsMenuRegistered(MenuName) == false)
	{
		UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu(MenuName);
		ToolMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* DynamicMenu)
		{
			UDataHierarchyEditorMenuContext* ContextObject = DynamicMenu->Context.FindContext<UDataHierarchyEditorMenuContext>();
			if(!ensure(ContextObject))
			{
				return;
			}

			TArray<TSubclassOf<UHierarchyElement>> AdditionalTypes = ContextObject->Widget.Pin()->HierarchyViewModel->GetAdditionalTypesToAddInUi();
			for(TSubclassOf<UHierarchyElement> Type : AdditionalTypes)
			{
				// We use the first available source. There is guaranteed to be one, although there could be multiple.
				FToolMenuSection& Section = DynamicMenu->FindOrAddSection(NAME_None);

				FToolUIAction UIAction;
				UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda([ContextObject, Type](const FToolMenuContext& Context)
				{
					TSharedPtr<SDataHierarchyEditor> DataHierarchyEditor = ContextObject->Widget.Pin();
					TArray<TSharedPtr<FHierarchyElementViewModel>> SelectedItems;
					DataHierarchyEditor->HierarchyTreeView->GetSelectedItems(SelectedItems);

					TSharedPtr<FHierarchyElementViewModel> ParentViewModel;
					if(SelectedItems.Num() == 1)
					{
						ParentViewModel = SelectedItems[0];
					}
					else
					{
						ParentViewModel = DataHierarchyEditor->HierarchyViewModel->GetHierarchyRootViewModel();
					}

					if(ParentViewModel->CanContain(Type).bResult)
					{
						ParentViewModel->AddChild(Type);
					}
				});
				UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda([ContextObject, Type](const FToolMenuContext& Context)
				{
					TSharedPtr<SDataHierarchyEditor> DataHierarchyEditor = ContextObject->Widget.Pin();
					TArray<TSharedPtr<FHierarchyElementViewModel>> SelectedItems;
					DataHierarchyEditor->HierarchyTreeView->GetSelectedItems(SelectedItems);

					TSharedPtr<FHierarchyElementViewModel> ParentViewModel;
					if(SelectedItems.Num() == 1)
					{
						ParentViewModel = SelectedItems[0];
					}
					else
					{
						ParentViewModel = DataHierarchyEditor->HierarchyViewModel->GetHierarchyRootViewModel();
					}
					
					return ParentViewModel->CanContain(Type).bResult;
				});

				TAttribute<FText> TooltipAttribute = TAttribute<FText>::CreateLambda([ContextObject, Type]()
				{
					TSharedPtr<SDataHierarchyEditor> DataHierarchyEditor = ContextObject->Widget.Pin();
					TArray<TSharedPtr<FHierarchyElementViewModel>> SelectedItems;
					DataHierarchyEditor->HierarchyTreeView->GetSelectedItems(SelectedItems);

					TSharedPtr<FHierarchyElementViewModel> ParentViewModel;

					if(SelectedItems.Num() == 1)
					{
						ParentViewModel = SelectedItems[0];
					}
					else
					{
						ParentViewModel = DataHierarchyEditor->HierarchyViewModel->GetHierarchyRootViewModel();
					}
					
					if(TOptional<FText> DynamicText = ParentViewModel->CanContain(Type).UserFeedback)
					{
						FText BaseText = FText::AsCultureInvariant("{0}\n\n{1}");
						FText Text = DynamicText.GetValue();
						return FText::FormatOrdered(BaseText, Type->GetToolTipText(), Text);
					}

					return Type->GetToolTipText();
				});
				
				FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(Type->GetFName(), Type->GetDisplayNameText(), TooltipAttribute, FSlateIconFinder::FindIconForClass(Type), UIAction);
				Section.AddEntry(Entry);
			}
		}));
	}

	UDataHierarchyEditorMenuContext* MenuContext = NewObject<UDataHierarchyEditorMenuContext>();
	MenuContext->Widget = SharedThis(this);
	
	return UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext(MenuContext));

}

void SDataHierarchyEditor::SetActiveSourceSection(TSharedPtr<FHierarchySectionViewModel> Section)
{
	if(HierarchyViewModel->SupportsSourcePanel() == false)
	{
		return;
	}
	
	ActiveSourceSection = Section;	
	RefreshSourceView(true);
	OnSourceSectionActivated(Section);
}

TSharedPtr<FHierarchySectionViewModel> SDataHierarchyEditor::GetActiveSourceSection() const
{
	return ActiveSourceSection.IsValid() ? ActiveSourceSection.Pin() : nullptr;
}

UHierarchySection* SDataHierarchyEditor::GetActiveSourceSectionData() const
{
	return ActiveSourceSection.IsValid() ? ActiveSourceSection.Pin()->GetDataMutable<UHierarchySection>() : nullptr;
}

void SDataHierarchyEditor::OnSelectionChanged(TSharedPtr<FHierarchyElementViewModel> SelectedHierarchyElement, ESelectInfo::Type Type) const
{
	// Guard against re-entrant calls: ClearSelection on the other tree view triggers OnSelectionChanged again,
	// and rebuilding the name area widget while a previous one is still being attached causes a crash.
	if(bIsUpdatingSelection)
	{
		return;
	}
	TGuardValue<bool> SelectionGuard(bIsUpdatingSelection, true);

	SelectedDetailsPanelItemViewModels.Reset();
	if(!DetailsPanel.IsValid())
	{
		return;
	}

	// Gather the full selection from the appropriate tree view
	TArray<TSharedPtr<FHierarchyElementViewModel>> AllSelected;
	bool bIsFromHierarchy = true;

	if(SelectedHierarchyElement.IsValid())
	{
		// Section activation
		if(SelectedHierarchyElement->GetData()->IsA<UHierarchySection>())
		{
			HierarchyTreeView->ClearSelection();
			AllSelected = {SelectedHierarchyElement};
		}
		else if(SelectedHierarchyElement->IsForHierarchy())
		{
			AllSelected = HierarchyTreeView->GetSelectedItems();
			if(HierarchyViewModel->SupportsSourcePanel())
			{
				SourceTreeView->ClearSelection();
			}
		}
		else
		{
			bIsFromHierarchy = false;
			AllSelected = HierarchyViewModel->SupportsSourcePanel() ? SourceTreeView->GetSelectedItems() : TArray<TSharedPtr<FHierarchyElementViewModel>>();
			HierarchyTreeView->ClearSelection();
		}
	}

	// When multi-edit is disabled, only show the last selected element (original behavior)
	if(!HierarchyViewModel->SupportsMultiEditInDetailsPanel() && AllSelected.Num() > 1)
	{
		AllSelected = {SelectedHierarchyElement};
	}

	// Filter to editable elements
	TArray<TSharedPtr<FHierarchyElementViewModel>> EditableElements;
	TArray<UObject*> ObjectsForEditing;
	for(const TSharedPtr<FHierarchyElementViewModel>& Element : AllSelected)
	{
		if(Element.IsValid() && Element->AllowEditingInDetailsPanel())
		{
			UObject* Obj = Element->GetObjectForEditing();
			if(Obj)
			{
				Obj->SetFlags(RF_Transactional);
				ObjectsForEditing.Add(Obj);
				EditableElements.Add(Element);
			}
		}
	}

	// Fallback to root if nothing is selected
	if(EditableElements.IsEmpty() && HierarchyViewModel->SelectRootInsteadOfNone())
	{
		TSharedPtr<FHierarchyRootViewModel> RootVM = HierarchyViewModel->GetHierarchyRootViewModel();
		UObject* RootObj = RootVM->GetObjectForEditing();
		if(RootObj)
		{
			RootObj->SetFlags(RF_Transactional);
			ObjectsForEditing.Add(RootObj);
			EditableElements.Add(RootVM);
		}
	}

	if(ObjectsForEditing.Num() > 0)
	{
		if(ObjectsForEditing.Num() == 1)
		{
			DetailsPanel->SetObject(ObjectsForEditing[0]);
		}
		else
		{
			DetailsPanel->SetObjects(ObjectsForEditing);
		}

		for(const TSharedPtr<FHierarchyElementViewModel>& Element : EditableElements)
		{
			SelectedDetailsPanelItemViewModels.Add(Element);
		}
	}
	else
	{
		DetailsPanel->SetObject(nullptr);
	}

	// Update name area widget -- only when the delegate is bound and we have a valid selection, matching original behavior.
	// Calling SetNameAreaCustomContent more broadly causes widget double-ownership crashes.
	if(DetailsPanel.IsValid() && OnGenerateCustomDetailsPanelNameWidget.IsBound() && EditableElements.Num() > 0)
	{
		TSharedPtr<FHierarchyElementViewModel> PrimaryElement = EditableElements[0];

		TSharedRef<SHorizontalBox> NameWidget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.Lock"))
			.Visibility(PrimaryElement->IsEditableByUser().bResult ? EVisibility::Collapsed : EVisibility::Visible)
			.ToolTipText(PrimaryElement->IsEditableByUser().UserFeedback.Get(FText::GetEmpty()))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f)
		.VAlign(VAlign_Center)
		[
			OnGenerateCustomDetailsPanelNameWidget.Execute(PrimaryElement)
		];

		// (+N) badge for multi-selection with tooltip showing all selected elements
		if(EditableElements.Num() > 1)
		{
			TSharedRef<SVerticalBox> TooltipContent = SNew(SVerticalBox);
			for(const TSharedPtr<FHierarchyElementViewModel>& Element : EditableElements)
			{
				TooltipContent->AddSlot()
				.AutoHeight()
				.Padding(1.f)
				[
					OnGenerateCustomDetailsPanelNameWidget.Execute(Element)
				];
			}

			TSharedRef<SToolTip> BadgeTooltip = SNew(SToolTip)
			[
				TooltipContent
			];

			NameWidget->AddSlot()
			.AutoWidth()
			.Padding(4.f, 0.f, 2.f, 0.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::Format(NSLOCTEXT("DataHierarchyEditor", "MultiSelectBadge", "(+{0})"), EditableElements.Num() - 1))
				.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("SmallText"))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.ToolTip(BadgeTooltip)
			];
		}

		TSharedRef<SWidget> NameWidgetAsWidget = NameWidget;
		DetailsPanel->SetNameAreaCustomContent(NameWidgetAsWidget);
	}
}

void SDataHierarchyEditor::RunSourceSearch()
{
	if(!SourceSearchBox->GetText().IsEmpty())
	{
		OnSourceSearchTextChanged(SourceSearchBox->GetText());
	}
}

void SDataHierarchyEditor::OnSourceSearchTextChanged(const FText& Text)
{
	SourceSearchResults.Empty();
	FocusedSearchResult.Reset();
	SourceTreeView->ClearSelection();

	if(!Text.IsEmpty())
	{
		FString TextAsString = Text.ToString();;

		TArray<FSearchItem> SearchItems;
		GenerateSearchItems(SourceRootViewModel.ToSharedRef(), {}, SearchItems);
		
		for(const FSearchItem& SearchItem : SearchItems)
		{
			for(const FString& SearchTerm : SearchItem.GetEntry()->GetSearchTerms())
			{
				if(SearchTerm.Contains(TextAsString))
				{
					SourceSearchResults.Add(SearchItem);
				}
			}
		}

		ExpandSourceSearchResults();
		SelectNextSourceSearchResult();
	}
	else
	{
		SourceTreeView->ClearExpandedItems();
	}
}

void SDataHierarchyEditor::OnSourceSearchTextCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	bool bIsShiftDown = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
	if(CommitType == ETextCommit::OnEnter)
	{
		if(bIsShiftDown == false)
		{
			SelectNextSourceSearchResult();
		}
		else
		{
			SelectPreviousSourceSearchResult();
		}
	}
}

void SDataHierarchyEditor::OnSearchButtonClicked(SSearchBox::SearchDirection SearchDirection)
{
	if(SearchDirection == SSearchBox::Next)
	{
		SelectNextSourceSearchResult();
	}
	else
	{
		SelectPreviousSourceSearchResult();
	}
}

void SDataHierarchyEditor::GenerateSearchItems(TSharedRef<FHierarchyElementViewModel> Root, TArray<TSharedPtr<FHierarchyElementViewModel>> ParentChain, TArray<FSearchItem>& OutSearchItems)
{
	const TArray<TSharedPtr<FHierarchyElementViewModel>> FilteredChildren = Root->GetFilteredChildren();
	ParentChain.Add(Root);
	OutSearchItems.Add(FSearchItem{ParentChain});
	for(TSharedPtr<FHierarchyElementViewModel> Child : FilteredChildren)
	{
		GenerateSearchItems(Child.ToSharedRef(), ParentChain, OutSearchItems);
	}
}

void SDataHierarchyEditor::ExpandSourceSearchResults()
{
	SourceTreeView->ClearExpandedItems();

	for(FSearchItem& SearchResult : SourceSearchResults)
	{
		for(TSharedPtr<FHierarchyElementViewModel>& EntryInPath : SearchResult.Path)
		{
			SourceTreeView->SetItemExpansion(EntryInPath, true);
		}
	}
}

void SDataHierarchyEditor::SelectNextSourceSearchResult()
{
	if(SourceSearchResults.IsEmpty())
	{
		return;
	}
	
	if(!FocusedSearchResult.IsSet())
	{
		FocusedSearchResult = SourceSearchResults[0];
	}
	else
	{
		int32 CurrentSearchResultIndex = SourceSearchResults.Find(FocusedSearchResult.GetValue());
		if(SourceSearchResults.IsValidIndex(CurrentSearchResultIndex+1))
		{
			FocusedSearchResult = SourceSearchResults[CurrentSearchResultIndex+1];
		}
		else
		{
			FocusedSearchResult = SourceSearchResults[0];
		}
	}

	SourceTreeView->ClearSelection();
	SourceTreeView->RequestScrollIntoView(FocusedSearchResult.GetValue().GetEntry());
	SourceTreeView->SetItemSelection(FocusedSearchResult.GetValue().GetEntry(), true);
}

void SDataHierarchyEditor::SelectPreviousSourceSearchResult()
{
	if(SourceSearchResults.IsEmpty())
	{
		return;
	}
	
	if(!FocusedSearchResult.IsSet())
	{
		FocusedSearchResult = SourceSearchResults[0];
	}
	else
	{
		int32 CurrentSearchResultIndex = SourceSearchResults.Find(FocusedSearchResult.GetValue());
		if(SourceSearchResults.IsValidIndex(CurrentSearchResultIndex-1))
		{
			FocusedSearchResult = SourceSearchResults[CurrentSearchResultIndex-1];
		}
		else
		{
			FocusedSearchResult = SourceSearchResults[SourceSearchResults.Num()-1];
		}
	}

	SourceTreeView->ClearSelection();
	SourceTreeView->RequestScrollIntoView(FocusedSearchResult.GetValue().GetEntry());
	SourceTreeView->SetItemSelection(FocusedSearchResult.GetValue().GetEntry(), true);
}

TOptional<SSearchBox::FSearchResultData> SDataHierarchyEditor::GetSearchResultData() const
{
	if(SourceSearchResults.Num() > 0)
	{
		SSearchBox::FSearchResultData SearchResultData;
		SearchResultData.NumSearchResults = SourceSearchResults.Num();

		if(FocusedSearchResult.IsSet())
		{
			// we add one just to make it look nicer as this is merely for cosmetic purposes
			SearchResultData.CurrentSearchResultIndex = SourceSearchResults.Find(FocusedSearchResult.GetValue()) + 1;
		}
		else
		{
			SearchResultData.CurrentSearchResultIndex = INDEX_NONE;
		}

		return SearchResultData;
	}

	return TOptional<SSearchBox::FSearchResultData>();
}

void SDataHierarchyEditor::ExpandEntriesByDefault(TSharedPtr<STreeView<TSharedPtr<FHierarchyElementViewModel>>> TreeView) const
{
	TArray<TSharedPtr<FHierarchyElementViewModel>> EntriesToProcess(TreeView->GetRootItems());
	while (EntriesToProcess.Num() > 0)
	{
		TSharedPtr<FHierarchyElementViewModel> EntryToProcess = EntriesToProcess[0];
		EntriesToProcess.RemoveAtSwap(0);

		if (EntryToProcess->IsExpandedByDefault())
		{
			TreeView->SetItemExpansion(EntryToProcess, true);
			EntriesToProcess.Append(EntryToProcess->GetFilteredChildren());
		}
		else
		{
			TreeView->SetItemExpansion(EntryToProcess, false);
		}
	}
}

void SDataHierarchyEditor::SaveExpansionState(TSharedPtr<STreeView<TSharedPtr<FHierarchyElementViewModel>>> TreeView) const
{
	SavedExpansionByIdentity.Reset();
	SavedExpansionByObjectKey.Reset();

	TArray<TSharedPtr<FHierarchyElementViewModel>> EntriesToProcess(TreeView->GetRootItems());
	while (EntriesToProcess.Num() > 0)
	{
		TSharedPtr<FHierarchyElementViewModel> Entry = EntriesToProcess[0];
		EntriesToProcess.RemoveAtSwap(0);

		const bool bIsExpanded = TreeView->IsItemExpanded(Entry);
		const FHierarchyElementIdentity Identity = Entry->GetData()->GetPersistentIdentity();
		if (Identity.IsValid())
		{
			SavedExpansionByIdentity.Add(Identity, bIsExpanded);
		}
		else
		{
			SavedExpansionByObjectKey.Add(TObjectKey<const UHierarchyElement>(Entry->GetData()), bIsExpanded);
		}

		EntriesToProcess.Append(Entry->GetFilteredChildren());
	}
}

void SDataHierarchyEditor::RestoreExpansionState(TSharedPtr<STreeView<TSharedPtr<FHierarchyElementViewModel>>> TreeView) const
{
	TArray<TSharedPtr<FHierarchyElementViewModel>> EntriesToProcess(TreeView->GetRootItems());
	while (EntriesToProcess.Num() > 0)
	{
		TSharedPtr<FHierarchyElementViewModel> Entry = EntriesToProcess[0];
		EntriesToProcess.RemoveAtSwap(0);

		bool bShouldExpand = Entry->IsExpandedByDefault();

		const FHierarchyElementIdentity Identity = Entry->GetData()->GetPersistentIdentity();
		if (Identity.IsValid())
		{
			if (const bool* SavedState = SavedExpansionByIdentity.Find(Identity))
			{
				bShouldExpand = *SavedState;
			}
		}
		else
		{
			if (const bool* SavedState = SavedExpansionByObjectKey.Find(TObjectKey<const UHierarchyElement>(Entry->GetData())))
			{
				bShouldExpand = *SavedState;
			}
		}

		TreeView->SetItemExpansion(Entry, bShouldExpand);

		if (TreeView->IsItemExpanded(Entry))
		{
			EntriesToProcess.Append(Entry->GetFilteredChildren());
		}
	}
}

FReply SDataHierarchyEditor::HandleHierarchyRootDrop(const FGeometry& Geometry, const FDragDropEvent& DragDropEvent) const
{
	if(TSharedPtr<FHierarchyDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FHierarchyDragDropOp>())
	{
		TArray<FDraggedElementEntry> TopLevelEntries = DragDropOp->GetDragContext().GetTopLevelEntries();

		FScopedTransaction Transaction(LOCTEXT("Transaction_MultiDropOnRoot", "Dropped items on root"));

		for(const FDraggedElementEntry& Entry : TopLevelEntries)
		{
			TSharedPtr<FHierarchyElementViewModel> DroppedElement = Entry.Element.Pin();
			if(DroppedElement.IsValid())
			{
				HierarchyViewModel->GetHierarchyRootViewModel()->OnDroppedOn(DroppedElement, EItemDropZone::OntoItem);
			}
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FHierarchyElementViewModel::FResultWithUserFeedback SDataHierarchyEditor::CanDropOnRoot(TSharedPtr<FHierarchyElementViewModel> DraggedItem) const
{
	return HierarchyViewModel->GetHierarchyRootViewModel()->CanDropOn(DraggedItem, EItemDropZone::OntoItem);
}

bool SDataHierarchyEditor::OnCanDropOnRoot(TSharedPtr<FDragDropOperation> DragDropOperation) const
{
	if(DragDropOperation->IsOfType<FHierarchyDragDropOp>())
	{
		TSharedPtr<FHierarchyDragDropOp> HierarchyDragDropOp = StaticCastSharedPtr<FHierarchyDragDropOp>(DragDropOperation);
		TArray<FDraggedElementEntry> TopLevelEntries = HierarchyDragDropOp->GetDragContext().GetTopLevelEntries();
		for(const FDraggedElementEntry& Entry : TopLevelEntries)
		{
			TSharedPtr<FHierarchyElementViewModel> DraggedItem = Entry.Element.Pin();
			if(!DraggedItem.IsValid() || !CanDropOnRoot(DraggedItem).bResult)
			{
				return false;
			}
		}
		return TopLevelEntries.Num() > 0;
	}

	return false;
}

void SDataHierarchyEditor::OnRootDragEnter(const FDragDropEvent& DragDropEvent) const
{
	if(TSharedPtr<FHierarchyDragDropOp> HierarchyDragDropOp = DragDropEvent.GetOperationAs<FHierarchyDragDropOp>())
	{
		TArray<FDraggedElementEntry> TopLevelEntries = HierarchyDragDropOp->GetDragContext().GetTopLevelEntries();
		if(TopLevelEntries.Num() == 1)
		{
			FHierarchyElementViewModel::FResultWithUserFeedback CanPerformActionResults = CanDropOnRoot(TopLevelEntries[0].Element.Pin());
			HierarchyDragDropOp->SetDescription(CanPerformActionResults.UserFeedback.Get(FText::GetEmpty()));
		}
		else if(TopLevelEntries.Num() > 1)
		{
			HierarchyDragDropOp->SetDescription(FText::Format(NSLOCTEXT("DataHierarchyEditor", "MultiDropOnRoot", "Add {0} items to root"), TopLevelEntries.Num()));
		}
	}
}

void SDataHierarchyEditor::OnRootDragLeave(const FDragDropEvent& DragDropEvent) const
{
	if(TSharedPtr<FHierarchyDragDropOp> HierarchyDragDropOp = DragDropEvent.GetOperationAs<FHierarchyDragDropOp>())
	{
		HierarchyDragDropOp->SetDescription(FText::GetEmpty());
	}
}

FSlateColor SDataHierarchyEditor::GetRootIconColor() const
{
	if(FSlateApplication::Get().IsDragDropping())
	{
		if(FSlateApplication::Get().GetDragDroppingContent()->IsOfType<FHierarchyDragDropOp>())
		{
			TSharedPtr<FHierarchyDragDropOp> HierarchyDragDropOp = StaticCastSharedPtr<FHierarchyDragDropOp>(FSlateApplication::Get().GetDragDroppingContent());
			TArray<FDraggedElementEntry> TopLevelEntries = HierarchyDragDropOp->GetDragContext().GetTopLevelEntries();
			bool bAllCanDrop = TopLevelEntries.Num() > 0;
			for(const FDraggedElementEntry& Entry : TopLevelEntries)
			{
				if(!Entry.Element.IsValid() || !CanDropOnRoot(Entry.Element.Pin()).bResult)
				{
					bAllCanDrop = false;
					break;
				}
			}
			if(bAllCanDrop)
			{
				return FLinearColor(0.8f, 0.8f, 0.8f, 0.8f);
			}
		}
	}

	return FLinearColor(0.2f, 0.2f, 0.2f, 0.5f);
}

void SDataHierarchyEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	const UObject* EditedObject = PropertyChangedEvent.GetObjectBeingEdited(0);

	TSharedPtr<FHierarchyElementViewModel> MatchingViewModel;
	// We look for the view model corresponding to the edited object.
	if(EditedObject->IsA<UHierarchySection>())
	{
		if(SourceRootViewModel.IsValid())
		{
			TSharedPtr<FHierarchySectionViewModel>* FoundSourceSectionViewModel = SourceRootViewModel->GetSectionViewModels().FindByPredicate([EditedObject](TSharedPtr<FHierarchySectionViewModel> Candidate)
				{
					return Candidate->GetSectionData() == EditedObject;
				});
			
			if(FoundSourceSectionViewModel)
			{
				MatchingViewModel = *FoundSourceSectionViewModel;
			}
		}

		TSharedPtr<FHierarchySectionViewModel>* FoundHierarchySectionViewModel = HierarchyViewModel->GetHierarchyRootViewModel()->GetSectionViewModels().FindByPredicate([EditedObject](TSharedPtr<FHierarchySectionViewModel> Candidate)
			{
				return Candidate->GetSectionData() == EditedObject;
			});

		if(FoundHierarchySectionViewModel)
		{
			MatchingViewModel = *FoundHierarchySectionViewModel;
		}
	}
	// The object doesn't have to be a hierarchy element, since view models can provide external elements for editing.
	else
	{
		if(SourceRootViewModel.IsValid())
		{
			if(TSharedPtr<FHierarchyElementViewModel> FoundSourceViewModel = SourceRootViewModel->FindViewModelForObject(PropertyChangedEvent.GetObjectBeingEdited(0), true))
			{
				MatchingViewModel = FoundSourceViewModel;
			}
		}
		
		if(TSharedPtr<FHierarchyElementViewModel> FoundHierarchyViewModel = HierarchyViewModel->GetHierarchyRootViewModel()->FindViewModelForObject(PropertyChangedEvent.GetObjectBeingEdited(0), true))
		{
			MatchingViewModel = FoundHierarchyViewModel;
		}
	}
	
	if (MatchingViewModel.IsValid())
	{
		MatchingViewModel->PostEditChange(PropertyChangedEvent, PropertyThatChanged);
		
		TInstancedStruct<FHierarchyChangedPayload_ElementPropertyChanged> Payload = TInstancedStruct<FHierarchyChangedPayload_ElementPropertyChanged>::Make();
		Payload.GetMutable<FHierarchyChangedPayload_ElementPropertyChanged>().ElementViewModel = MatchingViewModel;
		Payload.GetMutable<FHierarchyChangedPayload_ElementPropertyChanged>().PropertyChangedEvent = &PropertyChangedEvent;
		Payload.GetMutable<FHierarchyChangedPayload_ElementPropertyChanged>().PropertyThatChanged = PropertyThatChanged;

		HierarchyViewModel->OnHierarchyChanged(Payload);
	}
}

#undef LOCTEXT_NAMESPACE
