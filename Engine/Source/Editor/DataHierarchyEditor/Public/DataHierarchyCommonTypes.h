// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataHierarchyCommonTypes.generated.h"

#define UE_API DATAHIERARCHYEDITOR_API

/** This struct is used to identify a given hierarchy element and can be based on guids and/or names.
 *  This is particularly useful when a hierarchy element represents an object or a property that is not owned by the hierarchy itself. */
USTRUCT()
struct FHierarchyElementIdentity
{
	GENERATED_BODY()

	FHierarchyElementIdentity() {}
	FHierarchyElementIdentity(TArray<FGuid> InGuids, TArray<FName> InNames) : Guids(InGuids), Names(InNames) {}
	
	/** An array of guids that have to be satisfied in order to match. */
	UPROPERTY()
	TArray<FGuid> Guids;

	/** Optionally, an array of names can be specified in place of guids. If guids & names are present, guids have to be satisfied first, then names. */
	UPROPERTY()
	TArray<FName> Names;

	bool IsValid() const
	{
		return Guids.Num() > 0 || Names.Num() > 0;
	}
	
	bool operator==(const FHierarchyElementIdentity& OtherIdentity) const
	{
		if(Guids.Num() != OtherIdentity.Guids.Num() || Names.Num() != OtherIdentity.Names.Num())
		{
			return false;
		}

		for(int32 GuidIndex = 0; GuidIndex < Guids.Num(); GuidIndex++)
		{
			if(Guids[GuidIndex] != OtherIdentity.Guids[GuidIndex])
			{
				return false;
			}
		}

		for(int32 NameIndex = 0; NameIndex < Names.Num(); NameIndex++)
		{
			if(!Names[NameIndex].IsEqual(OtherIdentity.Names[NameIndex]))
			{
				return false;
			}
		}

		return true;
	}

	bool operator!=(const FHierarchyElementIdentity& OtherIdentity) const
	{
		return !(*this == OtherIdentity);
	}
};

FORCEINLINE uint32 GetTypeHash(const FHierarchyElementIdentity& Identity)
{
	uint32 Hash = 0;
	
	for(const FGuid& Guid : Identity.Guids)
	{
		Hash = HashCombine(Hash, GetTypeHash(Guid));
	}
	
	for(const FName& Name : Identity.Names)
	{
		Hash = HashCombine(Hash, GetTypeHash(Name));
	}
	
	return Hash;
}

USTRUCT()
struct FDataHierarchyElementMetaData
{
	GENERATED_BODY()
	
	FDataHierarchyElementMetaData() {}
};

USTRUCT()
struct FDataHierarchyElementMetaData_SectionAssociation : public FDataHierarchyElementMetaData
{
	GENERATED_BODY()

	FDataHierarchyElementMetaData_SectionAssociation() {}
	
	UPROPERTY()
	TWeakObjectPtr<const class UHierarchySection> Section;
};

#undef UE_API
