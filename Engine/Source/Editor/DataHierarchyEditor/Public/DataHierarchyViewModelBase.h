// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PropertyEditorDelegates.h"
#include "ToolMenuSection.h"
#include "IPropertyRowGenerator.h"
#include "Misc/TransactionObjectEvent.h"
#include "EditorUndoClient.h"
#include "Widgets/Views/STableRow.h"
#include "TickableEditorObject.h"
#include "Engine/TimerHandle.h"
#include "Templates/SubclassOf.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "StructUtils/InstancedStruct.h"
#include "DataHierarchyCommonTypes.h"
#include "DataHierarchyViewModelBase.generated.h"

#define UE_API DATAHIERARCHYEDITOR_API

/** HIERARCHY EDITOR
 *	The hierarchy editor is a generic tool to organize and structure all kinds of data.
 *	It inherently supports sections, categories, and items. You can add your own items and customize how they are visualized.
 *	Each hierarchy element is a UObject, and some hierarchy elements will represent externally owned data.
 *	For example, categories and sections defined within the hierarchy are also owned by the hierarchy, but an item might represent a parameter defined elsewhere.
 *
 *  Each hierarchy element is pure data and should not reference externally owned data that could become invalid.
 *  To define per-element rules, each hierarchy element gets assigned one view model.
 *  
 *	To use the Hierarchy Editor, you need multiple things:
 *	1) A UHierarchyRoot object that the Hierarchy Editor uses to store the created hierarchy
 *	2) A UDataHierarchyViewModelBase-derived object that defines core hierarchy rules. This is the main object responsible for configuring your hierarchy.
 *	-- The UDataHierarchyViewModelBase-derived class has multiple virtual functions you need to override. The key functions are:
 *	--- a) GetHierarchyRoot(), pointing to the UHierarchyRoot object you created in 1)
 *	--- b) PrepareSourceItems(UHierarchyRoot* SourceRoot, ...), which you need to use to populate the list of elements to be organized.
 *	------ These are transient items and a transient root. You can also add sections and categories here if you wish to structure the source element view.
 *	------ To add a child, call NewObject with ParentElement as the outer (in a flat list the SourceRoot).
 *	------ Then use ParentElement->GetChildrenMutable().Add(NewElement).
 *	------ This way, you can write your own initialization functions
 *	------ You can also add categories and sections to the SourceRoot
 *	--- c) Optionally but likely: CreateCustomViewModelForData(UHierarchyElement* Element, ...), which is used to create and assign non-default view models for each hierarchy element
 *	------ You don't need to implement this, but any slightly advanced use case that requires modification to hierarchy logic will need to implement this
 *	3) An SHierarchyEditor widget, which takes in the UDataHierarchyViewModelBase-derived object you created in 2). This way, the hierarchy editor knows where to store data, and how to initialize.
 *	--- Additionally, it has arguments such as 'OnGenerateRowContentWidget', which gives you an ElementViewModel.
 *	--- After determining the type, you can use this to define widgets for each hierarchy element.
 *
 *	Tips and tricks:
 *	1) Each hierarchy element can have a FHierarchyElementIdentity consisting of guid(s) and/or name(s).
 *	--- This can be used to uniquely identify an element, navigate to it and so on.
 *	2) To deal with automated cleanup of stale hierarchy elements that represent external data, you can set a UHierarchyDataRefreshContext-derived object on the UDataHierarchyViewModelBase object.
 *	--- The idea is to create a new derived class with objects and properties of the system that uses the Hierarchy Editor.
 *	--- For example, in a graph based system, the DataRefreshContext object can point to the graph.
 *	--- In an FHierarchyElementViewModel's 'DoesExternalDataStillExist' function,
 *	--- you can access the DataRefreshContext to determine whether your externally represented hierarchy elements should still exist.
 *	--- If not, it is automatically deleted.
 *	3) In the details panel, You can edit the hierarchy elements themselves, or external objects 
 *
 *  To make use of the created hierarchy, you access the UHierarchyRoot object you created in 1), and query it for its children, sections etc.
 *  The HierarchyEditor does not define how to use the created hierarchy data in your own UI. It only lets you structure and edit data.
 *  How you use it from a 'data consumption' point of view is up to you.
 */

struct FHierarchyRootViewModel;

/** A base class that is used to refresh data that represents external data. Inherit from this class if you need more context data. */
UCLASS(MinimalAPI, Transient)
class UHierarchyDataRefreshContext : public UObject
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UHierarchyElement : public UObject
{
	GENERATED_BODY()

public:
	UE_API UHierarchyElement();
	virtual ~UHierarchyElement() override = default;

	UE_API TConstArrayView<const TObjectPtr<UHierarchyElement>> GetChildren() const;
	TArray<TObjectPtr<UHierarchyElement>>& GetChildrenMutable() { return Children; }

	template<class ChildClass>
	ChildClass* AddChild();

	template<class ChildClass>
	ChildClass* AddChild(TSubclassOf<UHierarchyElement> Class, int32 InsertIndex = INDEX_NONE);

	UE_API UHierarchyElement* FindChildWithIdentity(FHierarchyElementIdentity ChildIdentity, bool bSearchRecursively = false);

	UE_API UHierarchyElement* CopyAndAddItemAsChild(const UHierarchyElement& ItemToCopy);
	UE_API UHierarchyElement* CopyAndAddItemUnderParentIdentity(const UHierarchyElement& ItemToCopy, FHierarchyElementIdentity ParentIdentity);
	
	/** Remove a child with a given identity. Can be searched recursively. This function operates under the assumption there will be only one item with a given identity. */
	UE_API bool RemoveChildWithIdentity(FHierarchyElementIdentity ChildIdentity, bool bSearchRecursively = false);
	
	template<class ChildClass>
	bool DoesOneChildExist(bool bRecursive = false) const;

	template<class ChildClass>
	void GetChildrenOfType(TArray<ChildClass*>& Out, bool bRecursive = false) const;

	template<class PREDICATE_CLASS>
	void SortChildren(const PREDICATE_CLASS& Predicate, bool bRecursive = false);
	
	template<class PREDICATE_CLASS>
	void StableSortChildren(const PREDICATE_CLASS& Predicate, bool bRecursive = false);
	
	virtual FString ToString() const { return GetName(); }
	FText ToText() const { return FText::FromString(ToString()); }
	
	/** An identity can be optionally set to create a mapping from previously existing guids or names to hierarchy items that represent them. */
	void SetIdentity(FHierarchyElementIdentity InIdentity) { Identity = InIdentity; }
	FHierarchyElementIdentity GetPersistentIdentity() const { return Identity; }
	UE_API TArray<FHierarchyElementIdentity> GetParentIdentities() const;
	UE_API TArray<const TObjectPtr<UHierarchyElement>> GetParentElements() const;
	
	/** Overridden modify method to also mark all children as modified */
	UE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;

	template<typename T>
	bool HasMetaDataOfType();

	template<typename T>
	const T* FindMetaDataOfType() const;
	
	template<typename T>
	T* FindMetaDataOfType();
	
	template<typename T>
	T FindMetaDataOfTypeOrDefault() const;
    	
	template<typename T>
	T* FindOrAddMetaDataOfType();

	template <typename T>
	bool DeleteMetaDataOfType();
	
protected:
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;

	UPROPERTY()
	TArray<TObjectPtr<UHierarchyElement>> Children;

	UPROPERTY()
	FHierarchyElementIdentity Identity;

	/** A map of metadata. Can be used for anything; an example of this is section association. */
	UPROPERTY()
	TMap<TObjectPtr<UStruct>, TInstancedStruct<FDataHierarchyElementMetaData>> MetaData;
	
	/** An optional guid; can be used if hierarchy items represent outside items */
	UPROPERTY()
	FGuid Guid_DEPRECATED;
};

template <class ChildClass>
ChildClass* UHierarchyElement::AddChild()
{
	ChildClass* NewChild = NewObject<ChildClass>(this);
	NewChild->SetFlags(RF_Transactional);

	GetChildrenMutable().Add(NewChild);

	return NewChild;
}

template <class ChildClass>
ChildClass* UHierarchyElement::AddChild(TSubclassOf<UHierarchyElement> Class, int32 InsertIndex)
{
	ChildClass* NewChild = NewObject<ChildClass>(this, Class);
	NewChild->SetFlags(RF_Transactional);

	if(InsertIndex == INDEX_NONE)
	{
		GetChildrenMutable().Add(NewChild);
	}
	else
	{
		GetChildrenMutable().Insert(NewChild, InsertIndex);
	}

	return NewChild;
}

template <class ChildClass>
bool UHierarchyElement::DoesOneChildExist(bool bRecursive) const
{
	for(UHierarchyElement* ChildElement : Children)
	{
		if(ChildElement->IsA<ChildClass>())
		{
			return true;
		}
	}

	if(bRecursive)
	{
		for(UHierarchyElement* ChildElement : Children)
		{
			if(ChildElement->DoesOneChildExist<ChildClass>(bRecursive))
			{
				return true;
			}
		}
	}

	return false;
}

template <class ChildClass>
void UHierarchyElement::GetChildrenOfType(TArray<ChildClass*>& Out, bool bRecursive) const
{
	for(UHierarchyElement* ChildElement : Children)
	{
		if(ChildElement->IsA<ChildClass>())
		{
			Out.Add(Cast<ChildClass>(ChildElement));
		}
	}

	if(bRecursive)
	{
		for(UHierarchyElement* ChildElement : Children)
		{
			ChildElement->GetChildrenOfType<ChildClass>(Out, bRecursive);
		}
	}
}

template <class PREDICATE_CLASS>
void UHierarchyElement::SortChildren(const PREDICATE_CLASS& Predicate, bool bRecursive)
{
	Children.Sort(Predicate);

	if(bRecursive)
	{
		for(TObjectPtr<UHierarchyElement> ChildElement : Children)
		{
			ChildElement->SortChildren(Predicate, bRecursive);
		}
	}
}

template <class PREDICATE_CLASS>
void UHierarchyElement::StableSortChildren(const PREDICATE_CLASS& Predicate, bool bRecursive)
{
	Children.StableSort(Predicate);

	if(bRecursive)
	{
		for(TObjectPtr<UHierarchyElement> ChildElement : Children)
		{
			ChildElement->StableSortChildren(Predicate, bRecursive);
		}
	}
}

template <typename T>
bool UHierarchyElement::HasMetaDataOfType()
{
	static_assert(TIsDerivedFrom<T, FDataHierarchyElementMetaData>::IsDerived, "T must be derived from FDataHierarchyElementMetaData");

	return MetaData.Contains(T::StaticStruct());
}

template <typename T>
const T* UHierarchyElement::FindMetaDataOfType() const
{
	static_assert(TIsDerivedFrom<T, FDataHierarchyElementMetaData>::IsDerived, "T must be derived from FDataHierarchyElementMetaData");
	
	if(MetaData.Contains(T::StaticStruct()))
	{
		return MetaData[T::StaticStruct()].template GetPtr<T>();
	}

	return nullptr;
}

template <typename T>
T* UHierarchyElement::FindMetaDataOfType()
{
	static_assert(TIsDerivedFrom<T, FDataHierarchyElementMetaData>::IsDerived, "T must be derived from FDataHierarchyElementMetaData");
	
	if(MetaData.Contains(T::StaticStruct()))
	{
		return MetaData[T::StaticStruct()].template GetMutablePtr<T>();
	}

	return nullptr;
}

template <typename T>
T UHierarchyElement::FindMetaDataOfTypeOrDefault() const
{
	static_assert(TIsDerivedFrom<T, FDataHierarchyElementMetaData>::IsDerived, "T must be derived from FDataHierarchyElementMetaData");
	static_assert(TIsConstructible<T>::Value, "T must be default constructible");
	
	if(MetaData.Contains(T::StaticStruct()))
	{
		return MetaData[T::StaticStruct()].template Get<T>();
	}

	return T();
}

template <typename T>
T* UHierarchyElement::FindOrAddMetaDataOfType()
{
	static_assert(TIsDerivedFrom<T, FDataHierarchyElementMetaData>::IsDerived, "T must be derived from FDataHierarchyElementMetaData");
	
	if(MetaData.Contains(T::StaticStruct()))
	{
		return MetaData[T::StaticStruct()].template GetMutablePtr<T>();
	}

	return MetaData.FindOrAdd(T::StaticStruct(), TInstancedStruct<T>::Make()).template GetMutablePtr<T>();
}

template <typename T>
bool UHierarchyElement::DeleteMetaDataOfType()
{
	static_assert(TIsDerivedFrom<T, FDataHierarchyElementMetaData>::IsDerived, "T must be derived from FDataHierarchyElementMetaData");

	if(MetaData.Contains(T::StaticStruct()))
	{
		return MetaData.Remove(T::StaticStruct()) != 0;
	}

	return false;
}

/** A minimal implementation of a section. */
UCLASS(MinimalAPI)
class UHierarchySection : public UHierarchyElement
{
	GENERATED_BODY()

public:
	UHierarchySection() {}

	void SetSectionName(FName InSectionName) { Section = InSectionName; }
	FName GetSectionName() const { return Section; }
	
	UE_API void SetSectionNameAsText(const FText& Text);
	FText GetSectionNameAsText() const { return FText::FromName(Section); }

	void SetTooltip(const FText& InTooltip) { Tooltip = InTooltip; }
	FText GetTooltip() const { return Tooltip; }

	virtual FString ToString() const override { return Section.ToString(); }
private:
	UPROPERTY()
	FName Section;

	/** The tooltip used when the user is hovering this section */
	UPROPERTY(EditAnywhere, Category = "Section", meta = (MultiLine = "true"))
	FText Tooltip;
};

/** UHierarchyRoot is used as the main object for serialization purposes, and a transient root is created automatically by the widget to populate the source list of items. */
UCLASS(MinimalAPI)
class UHierarchyRoot : public UHierarchyElement
{
	GENERATED_BODY()
public:
	UHierarchyRoot() {}
	virtual ~UHierarchyRoot() override = default;

	TConstArrayView<const TObjectPtr<UHierarchySection>> GetSectionData() const { return Sections; }
	TArray<TObjectPtr<UHierarchySection>>& GetSectionDataMutable() { return Sections; }

	UE_API TSet<FName> GetSectionNames() const;
	UE_API int32 GetSectionIndex(UHierarchySection* Section) const;
	
	UE_API UHierarchySection* FindSectionByIdentity(FHierarchyElementIdentity SectionIdentity);
	/** This will copy the section element itself */
	UE_API void DuplicateSectionFromOtherRoot(const UHierarchySection& SectionToCopy);
	
	UE_API virtual void EmptyAllData();
	
	UE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
protected:
	UE_API virtual void Serialize(FStructuredArchive::FRecord Record) override;
protected:
	UPROPERTY()
	TArray<TObjectPtr<UHierarchySection>> Sections;
};

/** A minimal implementation of an item. Inherit from this and add your own properties. */
UCLASS(MinimalAPI)
class UHierarchyItem : public UHierarchyElement
{
	GENERATED_BODY()
public:
	UHierarchyItem() {}
	virtual ~UHierarchyItem() override = default;
};

/** A category, potentially pointing at the section it belongs to. Only top-level categories can belong to sections by default.
 *  Inherit from this to add your own properties.  */
UCLASS(MinimalAPI)
class UHierarchyCategory : public UHierarchyElement
{
	GENERATED_BODY()
public:
	UHierarchyCategory() {}
	UHierarchyCategory(FName InCategory) : Category(InCategory) {}
	
	void SetCategoryName(FName NewCategory) { Category = NewCategory; }
	FName GetCategoryName() const { return Category; }

	FText GetCategoryAsText() const { return FText::FromName(Category); }
	FText GetTooltip() const { return Tooltip; }

	virtual FString ToString() const override { return Category.ToString(); }

	static UE_API FHierarchyElementIdentity ConstructIdentity();

protected:
	UE_API virtual void PostLoad() override;

private:
	UPROPERTY()
	FName Category;

	/** The tooltip used when the user is hovering this category */
	UPROPERTY(EditAnywhere, Category = "Category", meta = (MultiLine = "true"))
	FText Tooltip;

	UPROPERTY()
	TObjectPtr<UHierarchySection> Section_DEPRECATED = nullptr;
};

struct FHierarchyElementViewModel;
struct FHierarchySectionViewModel;

/** Entry wrapping a dragged element view model with computed context about its role in a multi-drag set. */
struct FDraggedElementEntry
{
	TWeakPtr<FHierarchyElementViewModel> Element;
	/** True if this element is a descendant of another element in the same drag set.
	 *  When true, drop execution should skip this entry -- the parent's drop already includes it. */
	bool bIsContainedInAnotherDraggedElement = false;
};

/** Context for a hierarchy drag-and-drop operation. Wraps the set of dragged elements with computed hierarchy metadata. */
struct FDataHierarchyDragDropContext
{
	FDataHierarchyDragDropContext() = default;
	UE_API ~FDataHierarchyDragDropContext();
	UE_API explicit FDataHierarchyDragDropContext(const TArray<FDraggedElementEntry>& InEntries);
	UE_API explicit FDataHierarchyDragDropContext(TArray<FDraggedElementEntry>&& InEntries);

	/** All dragged element entries, including those contained in other dragged elements. */
	UE_API const TArray<FDraggedElementEntry>& GetEntries() const;

	/** Returns only top-level entries (not contained in another dragged element). */
	UE_API TArray<FDraggedElementEntry> GetTopLevelEntries() const;
	/** Returns the number of top-level dragged elements. */
	UE_API int32 GetTopLevelCount() const;

	/** Returns the first/primary dragged element. */
	UE_API TWeakPtr<FHierarchyElementViewModel> GetPrimaryElement() const;

	UE_API int32 Num() const;
	UE_API bool IsEmpty() const;
private:
	TArray<FDraggedElementEntry> Entries;
};

/** Inherit from this to allow UI customization for your drag & drop operation by overriding CreateCustomDecorator. */
class FHierarchyDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FHierarchyDragDropOp, FDragDropOperation)

	UE_API FHierarchyDragDropOp(const FDataHierarchyDragDropContext& InDragContext);

	virtual void Construct() override { FDragDropOperation::Construct(); }
	UE_API virtual TSharedPtr<SWidget> GetDefaultDecorator() const override final;
	
	/** Override this custom decorator function to provide custom widget visuals. If not specified, you can still use Label and Description. */
	virtual TSharedRef<SWidget> CreateCustomDecorator() const { return SNullWidget::NullWidget; }

	/** Returns the first/primary dragged element. */
	TWeakPtr<FHierarchyElementViewModel> GetDraggedElement() const { return DragContext.GetPrimaryElement(); }
	/** Returns the full drag context. */
	const FDataHierarchyDragDropContext& GetDragContext() const { return DragContext; }

	void SetLabel(FText InText) { Label = InText; }
	FText GetLabel() const { return Label; }

	void SetDescription(FText InText) { Description = InText; }
	FText GetDescription() const { return Description; }

	void SetFromSourceList(bool bInFromSourceList) { bFromSourceList = bInFromSourceList; }
	bool GetIsFromSourceList() const { return bFromSourceList; }
protected:
	FDataHierarchyDragDropContext DragContext;

	/** Label will be displayed if no custom decorator has been specified. */
	FText Label;
	/** Useful for runtime tweaking of the tooltip based on what we are hovering. Always displayed if not-empty */
	FText Description;
	/** If the drag drop op is from the source list, we can further customize the actions */
	bool bFromSourceList = false;
};

UCLASS(MinimalAPI, BlueprintType)
class UHierarchyMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<class UDataHierarchyViewModelBase> HierarchyViewModel;
	TArray<TSharedPtr<FHierarchyElementViewModel>> MenuHierarchyElements;
};

/** Generic payload struct for when an element in the hierarchy has changed. */
USTRUCT()
struct FHierarchyChangedPayload
{
	GENERATED_BODY()
	
	FHierarchyChangedPayload() = default;
	FHierarchyChangedPayload(FString InMessage) : Message(InMessage) {}
	
	/** An optional message that can be set. */
	FString Message;
};

USTRUCT()
struct FHierarchyChangedPayload_ElementsAdded : public FHierarchyChangedPayload
{
	GENERATED_BODY()
	
	TArray<TSharedPtr<FHierarchyElementViewModel>> AddedElementViewModels;
};

USTRUCT()
struct FHierarchyChangedPayload_ElementsMoved : public FHierarchyChangedPayload
{
	GENERATED_BODY()
	
	TArray<TSharedPtr<FHierarchyElementViewModel>> MovedElementViewModels;
};

USTRUCT()
struct FHierarchyChangedPayload_ElementsDeleted : public FHierarchyChangedPayload
{
	GENERATED_BODY()

	TArray<TSharedPtr<FHierarchyElementViewModel>> DeletedElementViewModels;
};

USTRUCT()
struct FHierarchyChangedPayload_ElementPropertyChanged : public FHierarchyChangedPayload
{
	GENERATED_BODY()
	
	TSharedPtr<FHierarchyElementViewModel> ElementViewModel;
	const FPropertyChangedEvent* PropertyChangedEvent = nullptr;
	FProperty* PropertyThatChanged = nullptr;
};

USTRUCT()
struct FHierarchyChangedPayload_ElementNameChanged : public FHierarchyChangedPayload
{
	GENERATED_BODY()
	
	TSharedPtr<FHierarchyElementViewModel> ElementViewModel;
	FName OldName;
	FName NewName;
};

/** The main controller class for the SHierarchyEditor widget. Defines core hierarchy rules.
 *  Inherit from this and override the required virtual functions, instantiate an object, Initialize it and pass it to the SHierarchyEditor widget. */
UCLASS(MinimalAPI, Abstract, Transient)
class UDataHierarchyViewModelBase : public UObject, public FSelfRegisteringEditorUndoClient
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnHierarchyChanged, const TInstancedStruct<FHierarchyChangedPayload>&)
	DECLARE_DELEGATE_OneParam(FOnSectionActivated, TSharedPtr<FHierarchySectionViewModel> Section)
	DECLARE_DELEGATE_OneParam(FOnRefreshViewRequested, bool bForceFullRefresh)
	DECLARE_DELEGATE_OneParam(FOnNavigateToElementIdentityInHierarchyRequested, FHierarchyElementIdentity Identity)
	DECLARE_DELEGATE_OneParam(FOnNavigateToElementInHierarchyRequested, TSharedPtr<FHierarchyElementViewModel> ElementViewModel)

	GENERATED_BODY()

	UE_API UDataHierarchyViewModelBase();
	UE_API virtual ~UDataHierarchyViewModelBase() override;

	/** Initialize is called automatically for you, but it is recommended to call it manually after creating the HierarchyViewModel in your own Initialize function. This lets you access external data. */
	UE_API void Initialize();
	/** Call Finalize manually when you no longer need the HierarchyViewModel. */
	UE_API void Finalize();
	
	bool IsInitialized() const { return bIsInitialized; }
	bool IsFinalized() const { return bIsFinalized; }
	bool IsValid() const { return IsInitialized() && !IsFinalized(); }

	UE_API FName GetContextMenuName () const;

	/** Creates view model hierarchy elements. To create custom view models, override CreateCustomViewModelForElement. */
	UE_API TSharedPtr<FHierarchyElementViewModel> CreateViewModelForElement(UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> Parent);
	/** Get the root view model associated with the hierarchy. */
	TSharedPtr<struct FHierarchyRootViewModel> GetHierarchyRootViewModel() const { return HierarchyRootViewModel; }
	
	/** Hierarchy items reflect the already edited hierarchy. This should generally be constructed from persistent serialized data. */
	UE_API const TArray<TSharedPtr<FHierarchyElementViewModel>>& GetHierarchyItems() const;

	TSharedPtr<struct FHierarchySectionViewModel> GetDefaultHierarchySectionViewModel() const { return DefaultHierarchySectionViewModel; }
public:
	/** The hierarchy root the widget is editing. This should point to persistent data stored somewhere else as the serialized root of the hierarchy. */
	virtual UHierarchyRoot* GetHierarchyRoot() const PURE_VIRTUAL(UDataHierarchyViewModelBase::GetHierarchyRoot, return nullptr;);
	
	/** Prepares the items we want to create a hierarchy for. Primary purpose is to add children to the source root to gather the items to display in the source panel.
	 * The root view model is also given as a way to forcefully sync view models to access additional functionality, if needed. Only used if UsesSourceItems is true. */
	virtual void PrepareSourceItems(UHierarchyRoot* SourceRoot, TSharedPtr<FHierarchyRootViewModel> SourceRootViewModel) {};

	/** Generic callback for when something in the hierarchy has changed. */
	UE_API virtual void OnHierarchyChanged(const TInstancedStruct<FHierarchyChangedPayload>& Payload);
	
	/** If specified, will add a button in the Data Hierarchy Editor that allows adding the given types from a menu rather than drag & dropping them into the hierarchy. */
	virtual TArray<TSubclassOf<UHierarchyElement>> GetAdditionalTypesToAddInUi() const { return TArray<TSubclassOf<UHierarchyElement>>(); }
	
	/** The outer for the transient source root creation can be overridden. */
	virtual UObject* GetOuterForSourceRoot() const { return GetTransientPackage(); }
	
	/** The class used for creating categories. You can subclass UHierarchyCategory to add new properties. */
	UE_API virtual TSubclassOf<UHierarchyCategory> GetCategoryDataClass() const;
	
	/** The class used for creating sections. You can subclass UHierarchySection to add new properties. */
	UE_API virtual TSubclassOf<UHierarchySection> GetSectionDataClass() const;
	/** Function to implement drag drop ops. FHierarchyDragDropOp is a default implementation. Inherit from it and override CreateCustomDecorator for custom UI. */
	UE_API virtual TSharedRef<FHierarchyDragDropOp> CreateDragDropOp(const FDataHierarchyDragDropContext& DragContext);
	/** This needs to return true if you want the details panel to show up. */
	virtual bool SupportsDetailsPanel() { return true; }
	/** If true, selecting multiple elements will show a multi-object details panel. If false, only the last selected element is shown. */
	virtual bool SupportsMultiEditInDetailsPanel() { return true; }

	/** If false, disables source item functionality (left panel). */
	virtual bool SupportsSourcePanel() const { return true; }

	/** The selection mode used for the hierarchy tree view. Override to change the default multi-selection behavior. */
	virtual ESelectionMode::Type GetHierarchySelectionMode() const { return ESelectionMode::Multi; }

	/** Attempts to select the root object when no object is selected. Can be used in conjunction with `GetObjectForEditing' to point at external objects. */
	virtual bool SelectRootInsteadOfNone() const { return false; }
	
	/** Overriding this allows to define details panel instance customizations for specific UClasses */
	virtual TArray<TTuple<UClass*, FOnGetDetailCustomizationInstance>> GetInstanceCustomizations() { return {}; }

protected:
	/** Additional commands can be specified overriding the SetupCommands function. */
	virtual void SetupCommands() {}
private:
	/** Lets you add some additional logic to the Initialize function. */
	virtual void InitializeInternal() {}
	virtual void FinalizeInternal() {}

	/** This function is used to determine custom view models for Hierarchy Elements. Called by CreateViewModelForElement. */
	virtual TSharedPtr<FHierarchyElementViewModel> CreateCustomViewModelForElement(UHierarchyElement* Element, TSharedPtr<FHierarchyElementViewModel> Parent) { return nullptr; }
	/** Override this to add custom context menu options. Defaults to common actions such as finding, renaming and deleting.
	 * The hierarchy elements to generate this menu for are contained in a UHierarchyMenuContext object accessible through the ToolMenu. */
	UE_API virtual void GenerateDynamicContextMenuInternal(UToolMenu* DynamicToolMenu);
public:
	const UHierarchyDataRefreshContext* GetRefreshContext() const { return RefreshContext; }
	/** Set the refresh context to easily allow Hierarchy Elements representing external data to access whether the external data still exists. */
	void SetRefreshContext(UHierarchyDataRefreshContext* InContext) { RefreshContext = InContext; }
	
	UE_API void AddElementUnderRoot(TSubclassOf<UHierarchyElement> NewChildClass, FHierarchyElementIdentity ChildIdentity);
	UE_API void AddCategory(TSharedPtr<FHierarchyElementViewModel> CategoryParent = nullptr);

	/** Delete all specified elements. */
	UE_API void DeleteElements(TArray<TSharedPtr<FHierarchyElementViewModel>> ViewModels);
	/** Special case for deleting a specific element based on its identity.
	 *  Useful for externally removing an element from the hierarchy when you don't have access to the view model.*/
	UE_API void DeleteElementWithIdentity(FHierarchyElementIdentity Identity);

	UE_API void NavigateToElementInHierarchy(const FHierarchyElementIdentity& HierarchyIdentity) const;
	UE_API void NavigateToElementInHierarchy(const TSharedRef<FHierarchyElementViewModel> HierarchyElement) const;
	
	/** Refreshes all data and widgets */
	UE_API void ForceFullRefresh();
	UE_API void ForceFullRefreshOnTimer();
	UE_API void RequestFullRefreshNextFrame();
	
	TSharedRef<FUICommandList> GetCommands() const { return Commands.ToSharedRef(); }
	
	UE_API void OnGetChildren(TSharedPtr<FHierarchyElementViewModel> Element, TArray<TSharedPtr<FHierarchyElementViewModel>>& OutChildren) const;
	
	UE_API void RefreshAllViews(bool bFullRefresh = false) const;
	UE_API void RefreshSourceView(bool bFullRefresh = false) const;
	UE_API void RefreshHierarchyView(bool bFullRefresh = false) const;
	UE_API void RefreshSectionsView() const;
	
	// Delegate that call functions from SHierarchyEditor
	FSimpleDelegate& OnRefreshSourceItemsRequested() { return RefreshSourceItemsRequestedDelegate; }
	FOnRefreshViewRequested& OnRefreshSourceView() { return RefreshSourceViewDelegate; }
	FOnRefreshViewRequested& OnRefreshHierarchyView() { return RefreshHierarchyWidgetDelegate; }
	FSimpleDelegate& OnRefreshSectionsView() { return RefreshSectionsViewDelegate; }

	// Delegates for external systems
	FOnHierarchyChanged& GetOnHierarchyChangedDelegate() { return OnHierarchyChangedDelegate; } 
	FOnRefreshViewRequested& OnRefreshViewRequested() { return RefreshAllViewsRequestedDelegate; }
	FOnNavigateToElementIdentityInHierarchyRequested& OnNavigateToElementIdentityInHierarchyRequested() { return OnNavigateToElementIdentityInHierarchyRequestedDelegate; }
	FOnNavigateToElementInHierarchyRequested& OnNavigateToElementInHierarchyRequested() { return OnNavigateToElementInHierarchyRequestedDelegate; }
	FSimpleDelegate& OnInitialized() { return OnInitializedDelegate; }

	// Sections
	UE_API void SetActiveHierarchySection(TSharedPtr<struct FHierarchySectionViewModel>);
	UE_API TSharedPtr<FHierarchySectionViewModel> GetActiveHierarchySectionViewModel() const;
	UE_API const UHierarchySection* GetActiveHierarchySectionData() const;
	UE_API bool IsHierarchySectionActive(const UHierarchySection* Section) const;
	FOnSectionActivated& OnHierarchySectionActivated() { return OnHierarchySectionActivatedDelegate; }

	UE_API FString OnElementToStringDebug(TSharedPtr<FHierarchyElementViewModel> ElementViewModel) const;

protected:	
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;
	UE_API virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const override;
private:
	UE_API bool FilterForHierarchySection(TSharedPtr<const FHierarchyElementViewModel> ViewModel) const;

	static UE_API void GenerateDynamicContextMenu(UToolMenu* ToolMenu);

	UE_API void ToolMenuRequestRename(const FToolMenuContext& Context) const;
	UE_API bool ToolMenuCanRequestRename(const FToolMenuContext& Context) const;

	UE_API void ToolMenuDelete(const FToolMenuContext& Context);
	UE_API bool ToolMenuCanDelete(const FToolMenuContext& Context) const;

	UE_API void ToolMenuNavigateTo(const FToolMenuContext& Context) const;
	UE_API bool ToolMenuCanNavigateTo(const FToolMenuContext& Context) const;

protected:
	UPROPERTY()
	TObjectPtr<UHierarchyRoot> HierarchyRoot;
	
	TSharedPtr<struct FHierarchyRootViewModel> HierarchyRootViewModel;

	TSharedPtr<FHierarchySectionViewModel> DefaultHierarchySectionViewModel;
	TWeakPtr<struct FHierarchySectionViewModel> ActiveHierarchySectionViewModel;

	TSharedPtr<FUICommandList> Commands;

	UPROPERTY(Transient)
	TObjectPtr<UHierarchyDataRefreshContext> RefreshContext = nullptr;
	
protected:
	// delegate collection to call UI functions
	FSimpleDelegate RefreshSourceItemsRequestedDelegate;
	FOnRefreshViewRequested RefreshAllViewsRequestedDelegate;
	FOnRefreshViewRequested RefreshSourceViewDelegate;
	FOnRefreshViewRequested RefreshHierarchyWidgetDelegate;
	FSimpleDelegate RefreshSectionsViewDelegate;
	FOnNavigateToElementIdentityInHierarchyRequested OnNavigateToElementIdentityInHierarchyRequestedDelegate;
	FOnNavigateToElementInHierarchyRequested OnNavigateToElementInHierarchyRequestedDelegate;
	
	FOnSectionActivated OnHierarchySectionActivatedDelegate;
	FOnSectionActivated OnSourceSectionActivatedDelegate;
	FOnHierarchyChanged OnHierarchyChangedDelegate;
	
	FSimpleDelegate OnInitializedDelegate;

	FTimerHandle FullRefreshNextFrameHandle;

private:
	UPROPERTY(Transient)
	bool bIsInitialized = false;
	
	UPROPERTY(Transient)
	bool bIsFinalized = false;

	/** We construct a transient all section that isn't actually contained within the hierarchy. */
	UPROPERTY(Transient)
	TObjectPtr<UHierarchySection> AllSection;
public:
	static FName AllSectionHierarchyObjectName;
	static FName AllSectionSourceObjectName;
	static FText AllSectionDisplayName;
};

/** The base view model for all elements in the hierarchy. There are four base view models inheriting from this; for roots, items, categories, and sections.
 *  When creating a new view model, you should inherit from one of those four base view models.
 */
struct FHierarchyElementViewModel : TSharedFromThis<FHierarchyElementViewModel>, public FTickableEditorObject
{
	DECLARE_DELEGATE(FOnSynced)
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnFilterChild, const TSharedPtr<const FHierarchyElementViewModel> Child);
	DECLARE_DELEGATE_OneParam(FOnChildRequestedDeletion, TSharedPtr<FHierarchyElementViewModel> Child)
	
	FHierarchyElementViewModel(UHierarchyElement* InElement, TSharedPtr<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel, bool bInIsForHierarchy)
		: Element(InElement)
		, Parent(InParent)
		, HierarchyViewModel(InHierarchyViewModel)
		, bIsForHierarchy(bInIsForHierarchy)
	{		
		
	}

	/** Can be implemented for additional logic that the constructor isn't valid for. */
	virtual void Initialize() {}
	
	UE_API virtual ~FHierarchyElementViewModel() override;
	
	UHierarchyElement* GetDataMutable() { return Element; }
	const UHierarchyElement* GetData() const { return Element; }
	
	template<class T>
	T* GetDataMutable() { return Cast<T>(Element); }
	
	template<class T>
	const T* GetData() const { return Cast<T>(Element); }
	
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual TStatId GetStatId() const override;
	
	virtual FString ToString() const { return Element->ToString(); }
	FText ToStringAsText() const { return FText::FromString(ToString()); }
	virtual TArray<FString> GetSearchTerms() const { return {ToString()} ;}

	UE_API void RefreshChildrenData();
	UE_API void SyncViewModelsToData();
	
	/** Every item view model can define its own sort order for its children. By default we put categories above items. */
	UE_API virtual void SortChildrenData();

	const TArray<TSharedPtr<FHierarchyElementViewModel>>& GetChildren() const { return Children; }
	TArray<TSharedPtr<FHierarchyElementViewModel>>& GetChildrenMutable() { return Children; }
	UE_API const TArray<TSharedPtr<FHierarchyElementViewModel>>& GetFilteredChildren() const;
	
	UE_API void AddChildFilter(FOnFilterChild InFilterChild);
	
	template<class DataClass, class ViewModelChildClass>
	void GetChildrenViewModelsForType(TArray<TSharedPtr<ViewModelChildClass>>& OutChildren, bool bRecursive = false) const;

	/** Returns the hierarchy depth via number of parents above. */
	UE_API int32 GetHierarchyDepth() const;
	UE_API bool HasParent(TSharedPtr<FHierarchyElementViewModel> ParentCandidate, bool bRecursive = false) const;
	
	UE_API TSharedRef<FHierarchyElementViewModel> DuplicateToThis(TSharedPtr<FHierarchyElementViewModel> ItemToDuplicate, int32 InsertIndex = INDEX_NONE);
	UE_API TSharedRef<FHierarchyElementViewModel> ReparentToThis(TSharedPtr<FHierarchyElementViewModel> ItemToMove, int32 InsertIndex = INDEX_NONE);

	/** Finds the view model for a given hierarchy element. Needs to be a child of the current element somewhere down the hierarchy. */
	UE_API TSharedPtr<FHierarchyElementViewModel> FindViewModelForChild(const UHierarchyElement* Child, bool bSearchRecursively = false) const;
	/** Finds the view model for a given hierarchy element identity. Needs to be a child of the current element somewhere down the hierarchy. */
	UE_API TSharedPtr<FHierarchyElementViewModel> FindViewModelForChild(FHierarchyElementIdentity ChildIdentity, bool bSearchRecursively = false) const;
	/** Finds the view model for a given object. Can be a hierarchy element, but view models can also represent other objects not part of the hierarchy. */
	UE_API TSharedPtr<FHierarchyElementViewModel> FindViewModelForObject(const UObject* Object, bool bSearchRecursively = false) const;
	
	UE_API int32 FindIndexOfChild(const UHierarchyElement* Child) const;
	UE_API int32 FindIndexOfDataChild(TSharedPtr<const FHierarchyElementViewModel> Child) const;
	UE_API int32 FindIndexOfDataChild(const UHierarchyElement* Child) const;

	UE_API TSharedPtr<FHierarchyElementViewModel> AddChild(TSubclassOf<UHierarchyElement> NewChildClass, int32 InsertIndex = INDEX_NONE, FHierarchyElementIdentity ChildIdentity = FHierarchyElementIdentity());
	
	/** Deleting will ask the parent to delete its child */
	UE_API void Delete();
	UE_API void DeleteChild(TSharedPtr<FHierarchyElementViewModel> Child);

	TWeakObjectPtr<UDataHierarchyViewModelBase> GetHierarchyViewModel() const { return HierarchyViewModel; }

	/** Returns a set result if the item can accept a drop either above/onto/below the item.  */
	UE_API TOptional<EItemDropZone> OnCanRowAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone, TSharedPtr<FHierarchyElementViewModel> Item);
	UE_API virtual FReply OnDroppedOnRow(const FDragDropEvent& DragDropEvent, EItemDropZone ItemDropZone, TSharedPtr<FHierarchyElementViewModel> Item);
	UE_API void OnRowDragLeave(const FDragDropEvent& DragDropEvent);

	struct FResultWithUserFeedback
	{
		FResultWithUserFeedback(bool bInCanPerform) : bResult(bInCanPerform) {}
		
		bool bResult = false;
		/** A message that can be used for user feedback. Will either be used in tooltips in the hierarchy editor or as popup message etc.. */
		TOptional<FText> UserFeedback;

		bool operator==(const bool& bOther) const
		{
			return bResult == bOther;
		}

		bool operator!=(const bool& bOther) const
		{
			return !(*this==bOther);
		}
	};

	/** Called when a property on the underlying data changes. */
	virtual void PostEditChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) {}
	
	/** Should return true if properties are supposed to be editable & needs to be true if typical operations should work on it (renaming, dragging, deleting etc.) */
	virtual FResultWithUserFeedback IsEditableByUser() { return FResultWithUserFeedback(false); }
	
	/** Needs to be true in order to allow drag & drop operations to parent items to this item */
	virtual FResultWithUserFeedback CanHaveChildren() const { return false; }
	
	/** Should return true if an item should be draggable. An uneditable item can not be dragged even if CanDragInternal returns true. */
	FResultWithUserFeedback CanDrag();
	
	/** Returns true if renamable */
	bool CanRename() { return IsEditableByUser().bResult && CanRenameInternal(); }

	void Rename(FName NewName);

	void RequestRename()
	{
		if(CanRename() && OnRequestRenameDelegate.IsBound())
		{
			bRenamePending = false;
			OnRequestRenameDelegate.Execute();
		}
	}

	void RequestRenamePending()
	{
		if(CanRename())
		{
			bRenamePending = true;
		}
	}
	
	/** Returns true if deletable */
	bool CanDelete() { return IsEditableByUser().bResult && CanDeleteInternal(); }

	UE_API bool IsExpandedByDefault() const;
	
	/** Returns true if the given item can be dropped on the given target area. */
	UE_API FResultWithUserFeedback CanDropOn(TSharedPtr<FHierarchyElementViewModel> DraggedElement, EItemDropZone ItemDropZone);

	/** Gets executed when an item was dropped on this. */
	UE_API void OnDroppedOn(TSharedPtr<FHierarchyElementViewModel> DroppedItem, EItemDropZone ItemDropZone);
	/** Lets you completely customize the OnDroppedOn behavior. Needs to a specific value if you override this. */
	virtual TOptional<FResultWithUserFeedback> CanDropOnOverride(TSharedPtr<FHierarchyElementViewModel> DraggedElement, EItemDropZone ItemDropZone) { return {}; }
	virtual TOptional<bool> OnDroppedOnOverride(TSharedPtr<FHierarchyElementViewModel> DroppedElement, EItemDropZone ItemDropZone) { return {}; }
	
	/** Returns true if this element can add a new element via the add menu. */
	FResultWithUserFeedback CanContain(TSubclassOf<UHierarchyElement> HierarchyElementType);
	
	/** For data cleanup that represents external data, this needs to return true in order for live cleanup to work. */
	virtual bool RepresentsExternalData() const { return false; }
	/** This function determines whether a hierarchy item that represents that external data should be maintained during data refresh
	 * Needs to be implemented if RepresentsExternalData return true.
	 * The context object can be used to add arbitrary data. */
	virtual bool DoesExternalDataStillExist(const UHierarchyDataRefreshContext* Context) const { return false; }

	/** The UObject we display in the details panel when this item is selected. By default it's the hierarchy element the view model represents. */
	virtual UObject* GetObjectForEditing() { return Element; }
	/** Source items are transient, which is why we don't allow editing by default.
	 * This is useful to override if source data points at actual data to edit. */
	virtual bool AllowEditingInDetailsPanel() const { return bIsForHierarchy; }

	bool IsForHierarchy() const { return bIsForHierarchy; }

	/** Override this to register dynamic context menu entries when right clicking a single hierarchy item */
	virtual void AppendDynamicContextMenuForSingleElement(UToolMenu* ToolMenu) {}

	FSimpleDelegate& GetOnRequestRename() { return OnRequestRenameDelegate; }
	FOnSynced& GetOnSynced() { return OnSyncedDelegate; }
	FOnChildRequestedDeletion& OnChildRequestedDeletion() { return OnChildRequestedDeletionDelegate; }

	/** Returns the parent view model of the hierarchical parent */
	TWeakPtr<FHierarchyElementViewModel> GetParent() { return Parent; }

	virtual const FTableRowStyle* GetRowStyle() const { return nullptr; }

protected:
	/** Override this to handle post-add logic (for example when dropping an element on another element, or using the add button). */
	virtual void PostAddFixup() { }
	
	/** Called after an element was dropped onto the calling element. Allows inserting specialized fix up. Not called if OnDroppedOnOverride was called.
	 *  Note: DroppedElementViewModel is the newly created view model _after_ the drop occured. */
	virtual void PostOnDroppedOn(TSharedPtr<FHierarchyElementViewModel> DroppedElementViewModel) { }

private:
	/** Should return true if this element can contain other elements as children. Can optionally specify a message for failure. */
	virtual FResultWithUserFeedback CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType) { return false; }
	
	/** Should return true if draggable. An optional message can be provided if false that will show as a slate notification. */
	virtual FResultWithUserFeedback CanDragInternal() { return false; }

	/** Should return true if renamable */
	virtual bool CanRenameInternal() { return false; }

	virtual void RenameInternal(FName NewName) {}
	
	/** Should return true if deletable. By default, we can delete items in the hierarchy, not in the source. */
	virtual bool CanDeleteInternal() { return IsForHierarchy(); }

	UE_API virtual bool IsExpandedByDefaultInternal() const;
	
	/** Should return true if the given drag drop operation is allowed to succeed. */
	UE_API virtual FResultWithUserFeedback CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedElement, EItemDropZone ItemDropZone);
private:	
	/** Optionally implement this to refresh dependent data. */
	virtual void RefreshChildrenDataInternal() {}
	/** Optionally implement this to further customize the view model sync process.
	 * An example for this is how the root view model handles sections, as sections exist outside the children hierarchy */
	virtual void SyncViewModelsToDataInternal() {}
	/** Optionally implement this to handle shutdown logic.
	 * An example for this is when a section gets deleted, it iterates over all categories to null out the associated section  */
	virtual void FinalizeInternal() {}

	UE_API void PropagateOnChildSynced();
protected:
	/** The hierarchy element this view model represents. Assumed valid while this view model exists. */
	UHierarchyElement* const Element;

	/** Parent should be valid for all instances of this struct except for root objects */
	TWeakPtr<FHierarchyElementViewModel> Parent;
	TArray<TSharedPtr<FHierarchyElementViewModel>> Children;
	
	TWeakObjectPtr<UDataHierarchyViewModelBase> HierarchyViewModel;
	
	TArray<FOnFilterChild> ChildFilters;
	mutable TArray<TSharedPtr<FHierarchyElementViewModel>> FilteredChildren;
	
	FSimpleDelegate OnRequestRenameDelegate;
	FOnSynced OnSyncedDelegate;
	FOnChildRequestedDeletion OnChildRequestedDeletionDelegate;
	
	bool bRenamePending = false;
	bool bIsForHierarchy = false;
	bool bIsSyncInProgress = false;

	struct FDefaultUserFeedback
	{
		static FText UneditableTarget;
		static FText TargetCantContainType;
		static FText AddElementFromSourceToHierarchy;
		static FText MoveElementWithinHierarchy;
	};
};

template <class DataClass, class ViewModelClass>
void FHierarchyElementViewModel::GetChildrenViewModelsForType(TArray<TSharedPtr<ViewModelClass>>& OutChildren, bool bRecursive) const
{
	for(auto& Child : Children)
	{
		if(Child->GetData()->IsA<DataClass>())
		{
			OutChildren.Add(StaticCastSharedPtr<ViewModelClass>(Child));
		}
	}

	if(bRecursive)
	{
		for(auto& Child : Children)
		{
			Child->GetChildrenViewModelsForType<DataClass, ViewModelClass>(OutChildren, bRecursive);
		}
	}
}

struct FHierarchyRootViewModel : FHierarchyElementViewModel
{
	DECLARE_DELEGATE(FOnSyncPropagated)
	DECLARE_DELEGATE(FOnSectionsChanged)
	DECLARE_DELEGATE_OneParam(FOnSingleSectionChanged, TSharedPtr<FHierarchySectionViewModel> AddedSection)
	
	FHierarchyRootViewModel(UHierarchyElement* InItem, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel, bool bInIsForHierarchy) : FHierarchyElementViewModel(InItem, nullptr, InHierarchyViewModel, bInIsForHierarchy) {}
	UE_API virtual ~FHierarchyRootViewModel() override;

	UE_API virtual void Initialize() override;

	virtual FResultWithUserFeedback CanHaveChildren() const override { return true; }

	UE_API TSharedPtr<struct FHierarchySectionViewModel> AddSection();
	UE_API void DeleteSection(TSharedPtr<FHierarchyElementViewModel> SectionViewModel);
	TArray<TSharedPtr<struct FHierarchySectionViewModel>>& GetSectionViewModels() { return SectionViewModels; }

	FOnSyncPropagated& OnSyncPropagated() { return OnSyncPropagatedDelegate; }

	/** General purpose delegate for when sections change */
	FOnSectionsChanged& OnSectionsChanged() { return OnSectionsChangedDelegate; }
	/** Delegates for when a section is added or removed */
	FOnSingleSectionChanged& OnSectionAdded() { return OnSectionAddedDelegate; }
	FOnSingleSectionChanged& OnSectionDeleted() { return OnSectionDeletedDelegate; }

	virtual void PreEditHierarchyStructure() {}
	virtual void PostEditHierarchyStructure() {}
protected:
	virtual FResultWithUserFeedback IsEditableByUser() override { return true; }
	UE_API virtual FResultWithUserFeedback CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElementType) override;
	UE_API virtual FResultWithUserFeedback CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedElement, EItemDropZone ItemDropZone) override;
private:
	UE_API void PropagateOnSynced();
	UE_API virtual void SyncViewModelsToDataInternal() override;
	UE_API virtual void PostOnDroppedOn(TSharedPtr<FHierarchyElementViewModel> DroppedElementViewModel) override;
private:
	TArray<TSharedPtr<struct FHierarchySectionViewModel>> SectionViewModels;

	FOnSyncPropagated OnSyncPropagatedDelegate;
	FOnSingleSectionChanged OnSectionAddedDelegate;
	FOnSingleSectionChanged OnSectionDeletedDelegate;
	FOnSectionsChanged OnSectionsChangedDelegate;
};

struct FHierarchySectionViewModel : FHierarchyElementViewModel
{
	UE_API FHierarchySectionViewModel(UHierarchySection* InItem, TSharedRef<FHierarchyRootViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel);
	virtual ~FHierarchySectionViewModel() override = default;

	/** A custom function to retrieve data; if it's the all section, we want to return nullptr as that's what it serializes as. */
	UE_API const UHierarchySection* GetSectionData() const;
	
	UE_API virtual FString ToString() const override;
	
	UE_API void SetSectionName(FName InSectionName);
	UE_API FName GetSectionName() const;

	UE_API void SetSectionNameAsText(const FText& Text);
	UE_API FText GetSectionNameAsText() const;
	UE_API FText GetSectionTooltip() const;

	void SetSectionImage(const FSlateBrush* InSectionImage) { SectionImage = InSectionImage; }
	virtual const FSlateBrush* GetSectionImageBrush() const { return SectionImage ? SectionImage : FAppStyle::GetNoBrush(); }
	
	bool IsAllSection() const { return GetData()->GetFName() == UDataHierarchyViewModelBase::AllSectionHierarchyObjectName || GetData()->GetFName() == UDataHierarchyViewModelBase::AllSectionSourceObjectName; }
protected:
	UE_API virtual bool AllowEditingInDetailsPanel() const override;

	/** Only hierarchy sections are editable */
	UE_API virtual FResultWithUserFeedback IsEditableByUser() override;
	/** Technically a section does not have 'children' but for drag & drop logic it is considered as such */
	virtual FResultWithUserFeedback CanHaveChildren() const override { return true; }
	UE_API virtual FResultWithUserFeedback CanDragInternal() override;
	
	/** We can only rename hierarchy sections */
	UE_API virtual bool CanRenameInternal() override;
	virtual void RenameInternal(FName NewName) override { GetDataMutable<UHierarchySection>()->SetSectionName(NewName); }

	UE_API virtual bool CanDeleteInternal() override;
	
	/** We override the default dropped on behavior given sections don't actually contain children. */
	UE_API virtual TOptional<bool> OnDroppedOnOverride(TSharedPtr<FHierarchyElementViewModel> DroppedItem, EItemDropZone ItemDropZone) override;
	UE_API virtual FResultWithUserFeedback CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElement) override;
	UE_API virtual FResultWithUserFeedback CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedItem, EItemDropZone ItemDropZone) override;

	UE_API virtual void FinalizeInternal() override;

private:
	const FSlateBrush* SectionImage = nullptr;
};

struct FHierarchyItemViewModel : FHierarchyElementViewModel
{
	FHierarchyItemViewModel(UHierarchyItem* InElement, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel) : FHierarchyElementViewModel(InElement, InParent, InHierarchyViewModel, InParent->IsForHierarchy()) {}
	
	virtual ~FHierarchyItemViewModel() override = default;

	virtual FResultWithUserFeedback IsEditableByUser() override { return FResultWithUserFeedback(true); }
	virtual FResultWithUserFeedback CanHaveChildren() const override { return false; }
	virtual FResultWithUserFeedback CanDragInternal() override { return true; }

	UE_API virtual FResultWithUserFeedback CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElement) override;
	UE_API virtual FResultWithUserFeedback CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel> DraggedItem, EItemDropZone ItemDropZone) override;

	UE_API virtual const FTableRowStyle* GetRowStyle() const override;
};

struct FHierarchyCategoryViewModel : FHierarchyElementViewModel
{
	FHierarchyCategoryViewModel(UHierarchyCategory* InCategory, TSharedRef<FHierarchyElementViewModel> InParent, TWeakObjectPtr<UDataHierarchyViewModelBase> InHierarchyViewModel) : FHierarchyElementViewModel(InCategory, InParent, InHierarchyViewModel, InParent->IsForHierarchy()) {}
	virtual ~FHierarchyCategoryViewModel() override= default;

	void SetCategoryName(FName InText) { GetDataMutable<UHierarchyCategory>()->SetCategoryName(InText); }
	FText GetCategoryName() const { return GetData<UHierarchyCategory>()->GetCategoryAsText(); }
	
	virtual FResultWithUserFeedback IsEditableByUser() override { return FResultWithUserFeedback(true); }
	virtual FResultWithUserFeedback CanHaveChildren() const override { return true; }
	virtual FResultWithUserFeedback CanDragInternal() override { return true; }
	virtual bool CanRenameInternal() override { return true; }
	virtual void RenameInternal(FName NewName) override { GetDataMutable<UHierarchyCategory>()->SetCategoryName(NewName); }

	UE_API virtual FResultWithUserFeedback CanContainInternal(TSubclassOf<UHierarchyElement> InHierarchyElement) override;
	UE_API virtual FResultWithUserFeedback CanDropOnInternal(TSharedPtr<FHierarchyElementViewModel>, EItemDropZone ItemDropZone) override;

	UE_API virtual const FTableRowStyle* GetRowStyle() const override;
};

class FSectionDragDropOp : public FHierarchyDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FSectionDragDropOp, FHierarchyDragDropOp)

	FSectionDragDropOp(TSharedPtr<FHierarchySectionViewModel> SectionViewModel) : FHierarchyDragDropOp(FDataHierarchyDragDropContext({FDraggedElementEntry{SectionViewModel}})) {}

	TWeakPtr<FHierarchySectionViewModel> GetDraggedSection() const { return StaticCastSharedPtr<FHierarchySectionViewModel>(GetDraggedElement().Pin()); }
private:
	UE_API virtual TSharedRef<SWidget> CreateCustomDecorator() const override;
};

#undef UE_API
